{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mesh Painting",
	"Description": "System for painting data onto meshes.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MeshPaintEditorMode",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshPaintingToolset",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "InterchangeEditor",
			"Enabled": true,
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		}
	],
	"IsExperimentalVersion": false
}