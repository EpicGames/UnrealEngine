// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MeshPaintEditorMode : ModuleRules
	{
        public MeshPaintEditorMode(ReadOnlyTargetRules Target) : base(Target)
		{
   
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Slate",
					"SlateCore",
					"EditorSubsystem"
				}
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
				    "Core",
				    "CoreUObject",
				    "Engine",
                    "InputCore",
					"EditorFramework",
                    "EditorWidgets",
					"UnrealEd",
					"InteractiveToolsFramework",
					"EditorInteractiveToolsFramework",
					"MeshPaintingToolset",
					"GeometryCollectionEngine",
					"PropertyEditor",
					"MainFrame",
					"DesktopPlatform",
                    "RenderCore",
                    "RHI",
					"ToolWidgets",
					"InterchangeEngine",
					"InterchangePipelines",
					"DynamicMesh",
					"GeometryCore",
					"ImageCore",
					"MeshConversionEngineTypes",
					"Landscape"
				}
            );

        }
    }
}