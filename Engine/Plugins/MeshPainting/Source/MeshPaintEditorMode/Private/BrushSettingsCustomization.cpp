// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrushSettingsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "PropertyRestriction.h"
#include "Engine/Texture2D.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "SWarningOrErrorBox.h"
#include "ScopedTransaction.h"
#include "InteractiveToolManager.h"
#include "MeshPaintMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintModeHelpers.h"
#include "MeshTexturePaintingTool.h"
#include "MeshVertexPaintingTool.h"
#include "ContentBrowserDelegates.h"
#include "GeometryCollection/GeometryCollectionComponent.h"

#define LOCTEXT_NAMESPACE "MeshPaintCustomization"

TSharedRef<SHorizontalBox> CreateColorChannelWidget(TSharedRef<IPropertyHandle> ChannelProperty)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ChannelProperty->CreatePropertyValueWidget()
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			ChannelProperty->CreatePropertyNameWidget()
		];
}


TSharedRef<IDetailCustomization> FMeshPaintModeSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMeshPaintModeSettingsCustomization);
}

void FMeshPaintModeSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& ResourceCategory = DetailLayout.EditCategory(TEXT("ResourceUsage"));
	ResourceCategory.SetSortOrder(0);

	ResourceCategory.AddCustomRow(LOCTEXT("SelectionStateRow", "Selection State"))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("SelectedComponentsName", "Selected Components"))
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text_Lambda([]() -> FText
						{
							const int32 NumSelectedComponents = UMeshPaintMode::GetMeshPaintMode()->GetSelectedComponents<UMeshComponent>().Num();
							return FText::Format(LOCTEXT("SelectionCount", "{0}"), FText::AsNumber(NumSelectedComponents));
						})
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
						.ColorAndOpacity(FLinearColor::Yellow)
						.Visibility_Lambda([this]()
						{
							FText Message;
							return UMeshPaintMode::GetMeshPaintMode()->GetSelectionWarning(Message) ? EVisibility::Visible : EVisibility::Hidden;
						})
						.ToolTipText_Lambda([this]()
						{
							FText Message;
							UMeshPaintMode::GetMeshPaintMode()->GetSelectionWarning(Message);
							return Message;
						})
				]
		];

	ResourceCategory.AddCustomRow(NSLOCTEXT("VertexPaintSettings", "VertexColorSizeRow", "Vertex Color Size"))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(NSLOCTEXT("VertexPaintSettings", "VertexColorSize", "Instance vertex color size"))
		]
		.ValueContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([]() -> FText
				{
					const uint32 SizeInKB = FMath::DivideAndRoundNearest(UMeshPaintMode::GetMeshPaintMode()->GetVertexDataSizeInBytes(), 1024u);
					return FText::Format(NSLOCTEXT("VertexPaintSettings", "VertexColorSizeValue", "{0} KB"), FText::AsNumber(SizeInKB));
				})
		];

	ResourceCategory.AddCustomRow(NSLOCTEXT("VertexPaintSettings", "TextureColorSizeRow", "Texture Resource Size"))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(NSLOCTEXT("VertexPaintSettings", "TextureColorSize", "Mesh paint texture resource size"))
		]
		.ValueContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([]() -> FText 
				{
					const uint32 SizeInKB = FMath::DivideAndRoundNearest(UMeshPaintMode::GetMeshPaintMode()->GetMeshPaintTextureResourceSizeInBytes(), 1024u);
					return FText::Format(NSLOCTEXT("VertexPaintSettings", "TextureColorSizeValue", "{0} KB"), FText::AsNumber(SizeInKB)); 
				})
		];
}


TSharedRef<IDetailCustomization> FMeshPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMeshPaintingSettingsCustomization);
}

void FMeshPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& ColorCategory = DetailLayout.EditCategory(TEXT("ColorPainting"));

	// Customize paint color with a swap button
	TSharedRef<IPropertyHandle> PaintColor = DetailLayout.GetProperty("PaintColor", UMeshPaintingToolProperties::StaticClass());
	PaintColor->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> EraseColor = DetailLayout.GetProperty("EraseColor", UMeshPaintingToolProperties::StaticClass());
	EraseColor->MarkHiddenByCustomization();

	{
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;

		IDetailPropertyRow& PaintColorProp = ColorCategory.AddProperty(PaintColor);
		PaintColorProp.GetDefaultWidgets(NameWidget, ValueWidget, false);
		FDetailWidgetRow& PaintRow = PaintColorProp.CustomWidget(true);
		PaintRow.NameContent()
		[
			NameWidget.ToSharedRef()
		];

		PaintRow.ValueContent()
		.MinDesiredWidth(250)
		.MaxDesiredWidth(0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(250.f)
				[
					ValueWidget.ToSharedRef()
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(NSLOCTEXT("VertexPaintSettings", "SwapColors", "Swap Paint and Erase Colors"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &FMeshPaintingSettingsCustomization::OnSwapColorsClicked, PaintColor, EraseColor)
				.ContentPadding(0)
				[
					SNew(SImage).Image(FAppStyle::GetBrush("MeshPaint.Swap"))
				]
			]
		];
	}
	{
		IDetailPropertyRow& EraseColorProp = ColorCategory.AddProperty(EraseColor);

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;

		FDetailWidgetRow& Row = EraseColorProp.CustomWidget(true);
		Row.ValueContent().MinDesiredWidth(250 - 16.f);
		EraseColorProp.GetDefaultWidgets(NameWidget, ValueWidget, Row, false);
	}
}

FReply FMeshPaintingSettingsCustomization::OnSwapColorsClicked(TSharedRef<IPropertyHandle> PaintColor, TSharedRef<IPropertyHandle> EraseColor)
{
	FScopedTransaction Transaction(NSLOCTEXT("MeshPaintSettings", "SwapColorsTransation", "Swap paint and erase colors"));

	GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SwapColors();
	UMeshVertexPaintingToolProperties* Settings = UMeshPaintMode::GetVertexToolProperties();
	if (Settings)
	{
		PaintColor->NotifyPostChange(EPropertyChangeType::ValueSet);
		EraseColor->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
	return FReply::Handled();
}


TSharedRef<IDetailCustomization> FVertexPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FVertexPaintingSettingsCustomization);
}

void FVertexPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& VertexCategory = DetailLayout.EditCategory(TEXT("VertexPainting"));
	VertexCategory.SetSortOrder(3);

	/** Add custom row for painting on specific LOD level with callbacks to the painter to update the data */
	TSharedRef<IPropertyHandle> LODPaintingEnabled = DetailLayout.GetProperty("bPaintOnSpecificLOD", UMeshVertexPaintingToolProperties::StaticClass());
	LODPaintingEnabled->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> LODPaintingIndex = DetailLayout.GetProperty("LODIndex", UMeshVertexPaintingToolProperties::StaticClass());
	LODPaintingIndex->MarkHiddenByCustomization();
	TSharedPtr<SWidget> LODIndexWidget = LODPaintingIndex->CreatePropertyValueWidget();

	VertexCategory.AddCustomRow(NSLOCTEXT("LODPainting", "LODPaintingLabel", "LOD Model Painting"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("LODPainting", "LODPaintingSetupLabel", "LOD Model Painting"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.ToolTipText(NSLOCTEXT("LODPainting", "LODPaintingSetupToolTip", "Allows for Painting Vertex Colors on Specific LOD Models."))
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([=]() -> ECheckBoxState 
			{
				return (UMeshPaintMode::GetVertexToolProperties() && UMeshPaintMode::GetVertexToolProperties()->bPaintOnSpecificLOD ? ECheckBoxState::Checked : ECheckBoxState::Unchecked); 
			})
			.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([=](ECheckBoxState State) 
			{ 
				if (UMeshVertexPaintingTool* VertexBrush = Cast<UMeshVertexPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)))
				{
					VertexBrush->LODPaintStateChanged(State == ECheckBoxState::Checked);
				}
			}))
		]
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled_Lambda([=]() -> bool 
			{
				return UMeshPaintMode::GetVertexToolProperties() ? UMeshPaintMode::GetVertexToolProperties()->bPaintOnSpecificLOD : false;  
			})
			.AllowSpin(true)
			.Value_Lambda([=]() -> int32 
			{
				return UMeshPaintMode::GetVertexToolProperties() ? UMeshPaintMode::GetVertexToolProperties()->LODIndex : 0; 
			})
			.MinValue(0)
			.MaxValue_Lambda([=]() -> int32 
			{ 
				if (UMeshVertexPaintingTool* VertexBrush = Cast<UMeshVertexPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)))
				{
					return VertexBrush->GetMaxLODIndexToPaint();
				}
				return INT_MAX;
			})
			.MaxSliderValue_Lambda([=]() -> int32 
			{ 
				if (UMeshVertexPaintingTool* VertexBrush = Cast<UMeshVertexPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)))
				{
					return VertexBrush->GetMaxLODIndexToPaint();
				}
				return INT_MAX;
			})
			.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([=](int32 Value) { UMeshPaintMode::GetVertexToolProperties()->LODIndex = Value; }))
			.OnValueCommitted(SNumericEntryBox<int32>::FOnValueCommitted::CreateLambda([=](int32 Value, ETextCommit::Type CommitType)
			{ 
				UMeshPaintMode::GetVertexToolProperties()->LODIndex = Value; 
				if (UMeshVertexPaintingTool* VertexBrush = Cast<UMeshVertexPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)))
				{
					VertexBrush->PaintLODChanged();
				}
			}))
		]
	];

	VertexCategory.AddCustomRow(NSLOCTEXT("LODPainting", "LODPaintingLabel", "LOD Model Painting"))
	.WholeRowContent()
	[
		SNew(SWarningOrErrorBox)
		.Visibility_Lambda([this]() -> EVisibility
		{
			if (UMeshVertexPaintingToolProperties* VertexProperties = UMeshPaintMode::GetVertexToolProperties())
			{
				return VertexProperties->bPaintOnSpecificLOD ? EVisibility::Collapsed : EVisibility::Visible;
			}
			return EVisibility::Collapsed;
		})
		.Message_Lambda([this]() -> FText
		{
			static const FText SkelMeshNotificationText = LOCTEXT("SkelMeshAssetPaintInfo", "Paint is propagated to Skeletal Mesh Asset(s)");
			static const FText StaticMeshNotificationText = LOCTEXT("StaticMeshAssetPaintInfo", "Paint is applied to all LODs");
			static const FText GeometryCollectionNotificationText = LOCTEXT("GeometryCollectionAssetPaintInfo", "Paint is propagated to Geometry Collection Asset(s), and Geometry Collection does not currently support LODs.");

			const bool bGeometryCollectionText = UMeshPaintMode::GetMeshPaintMode()->GetSelectedComponents<UGeometryCollectionComponent>().Num() > 0;
			const bool bSkelMeshText = UMeshPaintMode::GetMeshPaintMode()->GetSelectedComponents<USkeletalMeshComponent>().Num() > 0;
			const bool bLODPaintText = UMeshPaintMode::GetVertexToolProperties() ?  !UMeshPaintMode::GetVertexToolProperties()->bPaintOnSpecificLOD : false;
			return FText::Format(FTextFormat::FromString(TEXT("{0}{1}{2}{3}")), 
				bSkelMeshText ? SkelMeshNotificationText : FText::GetEmpty(),
				bGeometryCollectionText ? GeometryCollectionNotificationText : FText::GetEmpty(),
				bSkelMeshText && bLODPaintText ? FText::FromString(TEXT("\n")) : FText::GetEmpty(),
				bLODPaintText ? StaticMeshNotificationText : FText::GetEmpty()
				);
		})
	];
}


TSharedRef<IDetailCustomization> FVertexColorPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FVertexColorPaintingSettingsCustomization);
}

void FVertexColorPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FMeshPaintingSettingsCustomization::CustomizeDetails(DetailLayout);
	FVertexPaintingSettingsCustomization::CustomizeDetails(DetailLayout);

	IDetailCategoryBuilder& ColorCategory = DetailLayout.EditCategory(TEXT("ColorPainting"));
	ColorCategory.SetSortOrder(1);
	IDetailCategoryBuilder& BrushCategory = DetailLayout.EditCategory(TEXT("Brush"));
	BrushCategory.SetSortOrder(2);

	/** Creates a custom widget row containing all color channel flags */
	TSharedRef<IPropertyHandle> RedChannel = DetailLayout.GetProperty("bWriteRed", UMeshVertexColorPaintingToolProperties::StaticClass());
	RedChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> GreenChannel = DetailLayout.GetProperty("bWriteGreen", UMeshVertexColorPaintingToolProperties::StaticClass());
	GreenChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> BlueChannel = DetailLayout.GetProperty("bWriteBlue", UMeshVertexColorPaintingToolProperties::StaticClass());
	BlueChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> AlphaChannel = DetailLayout.GetProperty("bWriteAlpha", UMeshVertexColorPaintingToolProperties::StaticClass());
	AlphaChannel->MarkHiddenByCustomization();
	TArray<TSharedRef<IPropertyHandle>> Channels = { RedChannel, GreenChannel, BlueChannel, AlphaChannel };
	TSharedPtr<SHorizontalBox> ChannelsWidget;

	ColorCategory.AddCustomRow(NSLOCTEXT("VertexPaintSettings", "ChannelLabel", "Channels"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("VertexPaintSettings", "ChannelsLabel", "Channels"))
		.ToolTipText(NSLOCTEXT("VertexPaintSettings", "ChannelsToolTip", "Colors Channels which should be influenced during Painting."))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SAssignNew(ChannelsWidget, SHorizontalBox)
	];

	for (TSharedRef<IPropertyHandle> Channel : Channels)
	{
		ChannelsWidget->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			CreateColorChannelWidget(Channel)
		];
	}
}


TSharedRef<IDetailCustomization> FVertexWeightPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FVertexWeightPaintingSettingsCustomization);
}

void FVertexWeightPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FVertexPaintingSettingsCustomization::CustomizeDetails(DetailLayout);

	IDetailCategoryBuilder& WeightCategory = DetailLayout.EditCategory(TEXT("WeightPainting"));
	WeightCategory.SetSortOrder(1);
	IDetailCategoryBuilder& BrushCategory = DetailLayout.EditCategory(TEXT("Brush"));
	BrushCategory.SetSortOrder(2);

	// Brush colors not needed for weight painting.
	TSharedRef<IPropertyHandle> PaintColor = DetailLayout.GetProperty("PaintColor", UMeshPaintingToolProperties::StaticClass());
	PaintColor->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> EraseColor = DetailLayout.GetProperty("EraseColor", UMeshPaintingToolProperties::StaticClass());
	EraseColor->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> WeightTypeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshVertexWeightPaintingToolProperties, TextureWeightType));
	TSharedRef<IPropertyHandle> PaintWeightProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshVertexWeightPaintingToolProperties, PaintTextureWeightIndex));
	TSharedRef<IPropertyHandle> EraseWeightProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshVertexWeightPaintingToolProperties, EraseTextureWeightIndex));

	WeightTypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FVertexWeightPaintingSettingsCustomization::OnTextureWeightTypeChanged, WeightTypeProperty, PaintWeightProperty, EraseWeightProperty));

	static FText RestrictReason = NSLOCTEXT("VertexPaintSettings", "TextureIndexRestriction", "Unable to paint this Texture, change Texture Weight Type");
	BlendPaintEnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));
	PaintWeightProperty->AddRestriction(BlendPaintEnumRestriction.ToSharedRef());
	EraseWeightProperty->AddRestriction(BlendPaintEnumRestriction.ToSharedRef());

	OnTextureWeightTypeChanged(WeightTypeProperty, PaintWeightProperty, EraseWeightProperty);
}

void FVertexWeightPaintingSettingsCustomization::OnTextureWeightTypeChanged(TSharedRef<IPropertyHandle> WeightTypeProperty, TSharedRef<IPropertyHandle> PaintWeightProperty, TSharedRef<IPropertyHandle> EraseWeightProperty)
{
	UEnum* ImportTypeEnum = StaticEnum<EMeshPaintTextureIndex>();
	uint8 EnumValue = 0;
	WeightTypeProperty->GetValue(EnumValue);

	BlendPaintEnumRestriction->RemoveAll();
	for (uint8 EnumIndex = 0; EnumIndex < (ImportTypeEnum->GetMaxEnumValue() + 1); ++EnumIndex)
	{
		if ((EnumIndex + 1) > EnumValue)
		{
			FString EnumName = ImportTypeEnum->GetNameByValue(EnumIndex).ToString();
			EnumName.RemoveFromStart("EMeshPaintTextureIndex::");
			BlendPaintEnumRestriction->AddDisabledValue(EnumName);
		}
	}

	uint8 Value = 0;
	PaintWeightProperty->GetValue(Value);
	Value = FMath::Clamp<uint8>(Value, 0, EnumValue - 1);
	PaintWeightProperty->SetValue(Value);

	EraseWeightProperty->GetValue(Value);
	Value = FMath::Clamp<uint8>(Value, 0, EnumValue - 1);
	EraseWeightProperty->SetValue(Value);
}


TSharedRef<IDetailCustomization> FTexturePaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FTexturePaintingSettingsCustomization);
}

void FTexturePaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FMeshPaintingSettingsCustomization::CustomizeDetails(DetailLayout);

	IDetailCategoryBuilder& ColorCategory = DetailLayout.EditCategory(TEXT("ColorPainting"));
	ColorCategory.SetSortOrder(1);
	IDetailCategoryBuilder& BrushCategory = DetailLayout.EditCategory(TEXT("Brush"));
	BrushCategory.SetSortOrder(2);
	IDetailCategoryBuilder& TextureCategory = DetailLayout.EditCategory(TEXT("TexturePainting"));
	TextureCategory.SetSortOrder(4);

	TSharedRef<IPropertyHandle> RedChannel = DetailLayout.GetProperty("bWriteRed", UMeshTexturePaintingToolProperties::StaticClass());
	RedChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> GreenChannel = DetailLayout.GetProperty("bWriteGreen", UMeshTexturePaintingToolProperties::StaticClass());
	GreenChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> BlueChannel = DetailLayout.GetProperty("bWriteBlue", UMeshTexturePaintingToolProperties::StaticClass());
	BlueChannel->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> AlphaChannel = DetailLayout.GetProperty("bWriteAlpha", UMeshTexturePaintingToolProperties::StaticClass());
	AlphaChannel->MarkHiddenByCustomization();
	TArray<TSharedRef<IPropertyHandle>> Channels = { RedChannel, GreenChannel, BlueChannel, AlphaChannel };
	TSharedPtr<SHorizontalBox> ChannelsWidget;

	ColorCategory.AddCustomRow(NSLOCTEXT("VertexPaintSettings", "ChannelLabel", "Channels"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("VertexPaintSettings", "ChannelsLabel", "Channels"))
		.ToolTipText(NSLOCTEXT("VertexPaintSettings", "ChannelsToolTip", "Colors Channels which should be influenced during Painting."))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SAssignNew(ChannelsWidget, SHorizontalBox)
	];

	for (TSharedRef<IPropertyHandle> Channel : Channels)
	{
		ChannelsWidget->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			CreateColorChannelWidget(Channel)
		];
	}
}


TSharedRef<IDetailCustomization> FTextureColorPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FTextureColorPaintingSettingsCustomization);
}

void FTextureColorPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FTexturePaintingSettingsCustomization::CustomizeDetails(DetailLayout);
}


TSharedRef<IDetailCustomization> FTextureAssetPaintingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FTextureAssetPaintingSettingsCustomization);
}

void FTextureAssetPaintingSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	FTexturePaintingSettingsCustomization::CustomizeDetails(DetailLayout);

	IDetailCategoryBuilder& WorldChannelCategory = DetailLayout.EditCategory(TEXT("WorldChannels"));
	WorldChannelCategory.SetSortOrder(3);
	IDetailCategoryBuilder& TextureCategory = DetailLayout.EditCategory(TEXT("TexturePainting"));
	TextureCategory.SetSortOrder(4);

	TSharedRef<IPropertyHandle> UVChannel = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTextureAssetPaintingToolProperties, UVChannel));
	DetailLayout.EditDefaultProperty(UVChannel)->CustomWidget()
	.NameContent()
	[
		UVChannel->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.AllowSpin(true)
			.Value_Lambda([]() -> int32 
			{ 
				return UMeshPaintMode::GetTextureAssetToolProperties() ? UMeshPaintMode::GetTextureAssetToolProperties()->UVChannel : 0; 
			})
			.MinValue(0)
			.MaxValue_Lambda([]() -> int32 
			{
				return FMath::Max(0, GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetMinMaxUVChannelsToPaint()[1] - 1);
			})
			.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([=](int32 Value)
			{
				if (UMeshPaintMode::GetTextureAssetToolProperties())
				{
					UMeshPaintMode::GetTextureAssetToolProperties()->UVChannel = Value;
				}
			}))
			.OnValueCommitted(SNumericEntryBox<int32>::FOnValueCommitted::CreateLambda([](int32 Value, ETextCommit::Type CommitType)
			{
				if (UMeshPaintMode::GetTextureAssetToolProperties())
				{
					UMeshPaintMode::GetTextureAssetToolProperties()->UVChannel = Value;
				}
			}))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
			.ColorAndOpacity(FLinearColor::Yellow)
			.Visibility_Lambda([this]()
			{
				const int32 UVChannel = UMeshPaintMode::GetTextureAssetToolProperties() ? UMeshPaintMode::GetTextureAssetToolProperties()->UVChannel : 0;
				const int32 NumSafeUVChannels = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetMinMaxUVChannelsToPaint()[0];
				const bool bIsSafe = UVChannel < NumSafeUVChannels;
				return bIsSafe ? EVisibility::Hidden : EVisibility::Visible;
			})
			.ToolTipText_Lambda([this]() 
			{
				return LOCTEXT("UVChannelWarning", "Some selected components do not support the selected UV channel.\nThey will not receive painting.");
			})
		]
	];

	TSharedRef<IPropertyHandle> TextureProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMeshTextureAssetPaintingToolProperties, PaintTexture));
	TextureProperty->MarkHiddenByCustomization();

	TSharedPtr<SHorizontalBox> TextureWidget;
	FDetailWidgetRow& Row = TextureCategory.AddCustomRow(NSLOCTEXT("TexturePaintSetting", "TextureSearchString", "Texture"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("TexturePaintSettings", "PaintTextureLabel", "Paint Texture"))
		.ToolTipText(NSLOCTEXT("TexturePaintSettings", "PaintTextureToolTip", "Texture to Apply Painting to."))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SAssignNew(TextureWidget, SHorizontalBox)
	];

	/** Use a SObjectPropertyEntryBox to benefit from its functionality */
	TextureWidget->AddSlot()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(TextureProperty)
		.AllowedClass(UTexture2D::StaticClass())
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateUObject(Cast<UMeshTextureAssetPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)), &UMeshTextureAssetPaintingTool::ShouldFilterTextureAsset))
		.DisplayUseSelected(false)
		.ThumbnailPool(DetailLayout.GetThumbnailPool())
	];
}


#undef LOCTEXT_NAMESPACE
