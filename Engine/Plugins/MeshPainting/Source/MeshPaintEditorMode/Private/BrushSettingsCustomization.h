// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FPropertyRestriction;
class FReply;
class IPropertyHandle;

class FMeshPaintModeSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FMeshPaintingSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	FReply OnSwapColorsClicked(TSharedRef<IPropertyHandle> PaintColor, TSharedRef<IPropertyHandle> EraseColor);
};

class FVertexPaintingSettingsCustomization : public FMeshPaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FVertexColorPaintingSettingsCustomization : public FVertexPaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FVertexWeightPaintingSettingsCustomization : public FVertexPaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	/** Property restriction applied to blend paint enum dropdown box */
	TSharedPtr<FPropertyRestriction> BlendPaintEnumRestriction;

	/** Callback for when texture weight type changed so we can update restrictions */
	void OnTextureWeightTypeChanged(TSharedRef<IPropertyHandle> WeightTypeProperty, TSharedRef<IPropertyHandle> PaintWeightProperty, TSharedRef<IPropertyHandle> EraseWeightProperty);
};

class FTexturePaintingSettingsCustomization : public FMeshPaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FTextureColorPaintingSettingsCustomization : public FTexturePaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};

class FTextureAssetPaintingSettingsCustomization : public FTexturePaintingSettingsCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};