// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportVertexColorOptions.h"

#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"

#include "Components/StaticMeshComponent.h"

#include "MeshPaintHelpers.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportVertexColorOptions)

#define LOCTEXT_NAMESPACE "VertexColorImportOptions"

void SImportVertexColorOptionsWindow::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;

	Options = GetMutableDefault<UImportVertexColorOptions>();
	Options->LODIndex = 0;
	Options->UVIndex = 0;
	
	// Populate max UV index for each LOD in the mesh component
	const int32 NumLODs = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetNumberOfLODs(InArgs._Component);
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		Options->LODToMaxUVMap.Add(LODIndex, GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetNumberOfUVs(InArgs._Component, LODIndex) - 1 );
	}
	Options->NumLODs = NumLODs;
	// Can only import vertex colors to static mesh component instances
	Options->bCanImportToInstance = InArgs._Component->IsA<UStaticMeshComponent>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(Options);
	
	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("Import_CurrentFileTitle", "Current File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(InArgs._FullPath)
				]
			]
		]		

		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ImportButton, SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("VertexColorOptionWindow_Import", "Import"))
				.OnClicked(this, &SImportVertexColorOptionsWindow::OnImport)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("VertexColorOptionWindow_Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("VertexColorOptionWindow_Cancel_ToolTip", "Cancels importing Vertex Colors"))
				.OnClicked(this, &SImportVertexColorOptionsWindow::OnCancel)
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE // "VertexColorImportOptions"


