// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

#include "ImportVertexColorOptions.generated.h"

class SButton;

UCLASS()
class UImportVertexColorOptions : public UObject
{
	GENERATED_BODY()

public:
	UImportVertexColorOptions() : UVIndex(0), LODIndex(0), bRed(true), bBlue(true), bGreen(true), bAlpha(true), bImportToInstance(true), NumLODs(0), bCanImportToInstance(false) {}
	
	/** Texture Coordinate Channel to use for Sampling the Texture*/
	UPROPERTY(EditAnywhere, Category=Options)
	int UVIndex;

	/** LOD Index to import the Vertex Colors to */
	UPROPERTY(EditAnywhere, Category = Options)
	int LODIndex;

	/** Red Texture Channel */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bRed;

	/** Blue Texture Channel */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bBlue;

	/** Green Texture Channel */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bGreen;

	/** Alpha Texture Channel */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bAlpha;

	/** Whether or not to import the Vertex Colors to Mesh Component instance or the underlying Static Mesh */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bImportToInstance;

	TMap<int32, int32> LODToMaxUVMap;
	int32 NumLODs;

	UPROPERTY()
	bool bCanImportToInstance;

	FColor CreateColorMask() const
	{
		FColor Mask = FColor::Black;
		Mask.R = bRed ? 255 : 0;
		Mask.B = bBlue ? 255 : 0;
		Mask.G = bGreen ? 255 : 0;
		Mask.A = bAlpha ? 255 : 0;

		return Mask;
	}
};

class IDetailsView;
class UMeshComponent;

class SImportVertexColorOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImportVertexColorOptionsWindow)	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)
		SLATE_ARGUMENT(UMeshComponent*, Component)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnImport()
	{
		bShouldImport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}

	UImportVertexColorOptions* GetOptions() const
	{
		return Options;
	}

	SImportVertexColorOptionsWindow()
	{}
private:
	TWeakPtr< SWindow > WidgetWindow;
	TSharedPtr< SButton > ImportButton;
	bool bShouldImport;
	TSharedPtr<IDetailsView> DetailsView;
	UImportVertexColorOptions* Options;
};
