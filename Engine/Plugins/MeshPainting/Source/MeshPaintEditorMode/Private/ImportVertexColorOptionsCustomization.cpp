// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportVertexColorOptionsCustomization.h"
#include "ImportVertexColorOptions.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "VertexColorImportOptionsCustomization"

void FImportVertexColorOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	UImportVertexColorOptions* Options = GetMutableDefault<UImportVertexColorOptions>();

	const FName CategoryName = "Options";
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);
	
	/** Retrieve properties we need */
	TSharedRef<IPropertyHandle> UVProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, UVIndex));
	TSharedRef<IPropertyHandle> LODProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, LODIndex));
	TSharedRef<IPropertyHandle> InstanceImportProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, bImportToInstance));

	/** Hide default properties, this allows us to use default widgets */
	DetailBuilder.HideProperty(UVProperty);
	DetailBuilder.HideProperty(LODProperty);
	DetailBuilder.HideProperty(InstanceImportProperty);

	/** Custom numeric box for the LOD index to ensure we stay within max LOD and per-lod UV indices*/
	CategoryBuilder.AddCustomRow(LOCTEXT("LODPropertyLabel", "LOD Index"))
	.NameContent()
	[
		LODProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SNumericEntryBox<int32>)
		.AllowSpin(true)
		.MinValue(0)
		.Value_Lambda([=]() -> int32 { return Options->LODIndex; })
		.MaxValue_Lambda([=]() -> int32 { return Options->NumLODs - 1; })
		.MinSliderValue(0)
		.MaxSliderValue_Lambda([=]() -> int32 { return Options->NumLODs - 1; })
		.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([=](int32 Value)
		{
			Options->LODIndex = Value;
			Options->UVIndex = FMath::Min(Options->UVIndex, Options->LODToMaxUVMap.FindChecked(Value));
		}))
		.OnValueCommitted(SNumericEntryBox<int32>::FOnValueCommitted::CreateLambda([=](int32 Value, ETextCommit::Type CommitType)
		{
			Options->LODIndex = Value; 
			// Ensure that UV index is always valid for the given LOD
			Options->UVIndex = FMath::Min(Options->UVIndex, Options->LODToMaxUVMap.FindChecked(Value)); 
		}))
	];

	/** Custom numeric box for UV index to ensure it is in valid ranges for the current LOD */
	CategoryBuilder.AddCustomRow(LOCTEXT("UVPropertyLabel", "UV Index"))
	.NameContent()
	[
		UVProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SNumericEntryBox<int32>)
		.AllowSpin(true)
		.MinValue(0)
		.Value_Lambda([=]() -> int32 { return Options->UVIndex; })
		.MinSliderValue(0)		
		.MaxSliderValue_Lambda([=]() -> int32 { return Options->LODToMaxUVMap.FindChecked(Options->LODIndex); })
		.MaxValue_Lambda([=]() -> int32 { return Options->LODToMaxUVMap.FindChecked(Options->LODIndex); })
		.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([=](int32 Value) { Options->UVIndex = Value;	}))
		.OnValueCommitted(SNumericEntryBox<int32>::FOnValueCommitted::CreateLambda([=](int32 Value, ETextCommit::Type CommitType) { Options->UVIndex = Value;  }))
	];
	
	// Allign all color channels into one widget from left to right
	TSharedRef<IPropertyHandle> RedChannel = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, bRed));
	TSharedRef<IPropertyHandle> GreenChannel = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, bGreen));
	TSharedRef<IPropertyHandle> BlueChannel = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, bBlue));
	TSharedRef<IPropertyHandle> AlphaChannel = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UImportVertexColorOptions, bAlpha));
	TArray<TSharedRef<IPropertyHandle>> Channels = { RedChannel, GreenChannel, BlueChannel, AlphaChannel };
	TSharedPtr<SHorizontalBox> ChannelsWidget;

	CategoryBuilder.AddCustomRow(LOCTEXT("ChannelLabel", "Channels"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT( "ChannelsLabel", "Channels"))
		.ToolTipText(LOCTEXT( "ChannelsToolTip", "Colors Channels which should be Imported."))
		.Font(DetailBuilder.GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SAssignNew(ChannelsWidget, SHorizontalBox)
	];

	for (TSharedRef<IPropertyHandle> Channel : Channels)
	{
		DetailBuilder.HideProperty(Channel);
		ChannelsWidget->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			CreateColorChannelWidget(Channel)
		];
	}	
	
	/** Can only import vertex colors to static mesh component instances */
	if (Options->bCanImportToInstance)
	{
		CategoryBuilder.AddProperty(InstanceImportProperty, EPropertyLocation::Common);
	}
}

TSharedRef<SHorizontalBox> FImportVertexColorOptionsCustomization::CreateColorChannelWidget(TSharedRef<IPropertyHandle> ChannelProperty)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ChannelProperty->CreatePropertyValueWidget()

		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			ChannelProperty->CreatePropertyNameWidget()
		];
}

#undef LOCTEXT_NAMESPACE // "VertexColorImportOptionsCustomization"
