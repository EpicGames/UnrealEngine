// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class SHorizontalBox;

/** Customization for importing vertex colors from a texture see SImportVertexColorOptions */
class FImportVertexColorOptionsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FImportVertexColorOptionsCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
private:
	TSharedRef<SHorizontalBox> CreateColorChannelWidget(TSharedRef<class IPropertyHandle> ChannelProperty);
};
