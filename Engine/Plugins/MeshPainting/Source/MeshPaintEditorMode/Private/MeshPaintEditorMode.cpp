// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintEditorMode.h"
#include "Modules/ModuleManager.h"
#include "MeshPaintMode.h"
#include "EditorModeManager.h"
#include "MeshPaintModeCommands.h"
#include "BrushSettingsCustomization.h"
#include "PropertyEditorModule.h"
#include "MeshPaintAdapterFactory.h"
#include "MeshPaintSplineMeshAdapter.h"
#include "MeshPaintSkeletalMeshAdapter.h"
#include "MeshPaintGeometryCollectionAdapter.h"

class FMeshPaintEditorModeModule : public IModuleInterface
{
public:
	FMeshPaintEditorModeModule() 
	{
	}

	// FModuleInterface overrides
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

protected:
	void Register();
	void OnPostEngineInit();
	void Unregister();

	void OnMeshPaintModeButtonClicked();
	bool IsMeshPaintModeButtonEnabled();
	void RegisterGeometryAdapterFactory(TSharedRef<IMeshPaintComponentAdapterFactory> Factory);
	void UnregisterGeometryAdapterFactory(TSharedRef<IMeshPaintComponentAdapterFactory> Factory);
};

void FMeshPaintEditorModeModule::Register()
{
	// Note: factories must be registered in the order of most to least specialized!!
	// The factory lookup currently uses first found rather than determining most specialized. Due to this, the SplineMesh factory must be 
	// registered *before* the StaticMesh factory, since both factories will succeed in constructing adapters for USplineMeshComponent
	// since it derives from UStaticMeshComponent.
	RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintSplineMeshComponentAdapterFactory));
	RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintStaticMeshComponentAdapterFactory));
	RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintSkeletalMeshComponentAdapterFactory));
	RegisterGeometryAdapterFactory(MakeShareable(new FMeshPaintGeometryCollectionComponentAdapterFactory));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("MeshPaintModeSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshPaintModeSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("MeshVertexColorPaintingToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexColorPaintingSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("MeshVertexWeightPaintingToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexWeightPaintingSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("MeshTextureColorPaintingToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FTextureColorPaintingSettingsCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("MeshTextureAssetPaintingToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FTextureAssetPaintingSettingsCustomization::MakeInstance));
}


void FMeshPaintEditorModeModule::OnPostEngineInit()
{
	Register();
	FMeshPaintingToolActionCommands::RegisterAllToolActions();
	FMeshPaintEditorModeCommands::Register();
}

void FMeshPaintEditorModeModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMeshPaintEditorModeModule::OnPostEngineInit);
}


void FMeshPaintEditorModeModule::ShutdownModule()
{
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FMeshPaintingToolActionCommands::UnregisterAllToolActions();
	FMeshPaintEditorModeCommands::Unregister();

	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	Unregister();
}




void FMeshPaintEditorModeModule::Unregister()
{
}


void FMeshPaintEditorModeModule::OnMeshPaintModeButtonClicked()
{
	// *Important* - activate the mode first since FEditorModeTools::DeactivateMode will
	// activate the default mode when the stack becomes empty, resulting in multiple active visible modes.
	GLevelEditorModeTools().ActivateMode(GetDefault<UMeshPaintMode>()->GetID());

}



bool FMeshPaintEditorModeModule::IsMeshPaintModeButtonEnabled( )
{
	return true;
}

void  FMeshPaintEditorModeModule::RegisterGeometryAdapterFactory(TSharedRef<IMeshPaintComponentAdapterFactory> Factory)
{
	FMeshPaintComponentAdapterFactory::FactoryList.Add(Factory);
}

void  FMeshPaintEditorModeModule::UnregisterGeometryAdapterFactory(TSharedRef<IMeshPaintComponentAdapterFactory> Factory)
{
	FMeshPaintComponentAdapterFactory::FactoryList.Remove(Factory);
}

IMPLEMENT_MODULE(FMeshPaintEditorModeModule, MeshPaintEditorMode)
