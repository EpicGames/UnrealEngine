// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintMode.h"

#include "AssetCompilingManager.h"
#include "EditorModeManager.h"
#include "MeshPaintModeCommands.h"
#include "EditorViewportClient.h"
#include "GeometryCollection/GeometryCollectionComponent.h" // IWYU pragma: keep
#include "MeshPaintModeToolkit.h"
#include "IMeshPaintComponentAdapter.h"
#include "MeshPaintModeSettings.h"
#include "InteractiveToolManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Engine/SkeletalMesh.h"
#include "ComponentReregisterContext.h"
#include "Dialogs/Dialogs.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ScopedTransaction.h"
#include "PackageTools.h"
#include "MeshPaintAdapterFactory.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintModeHelpers.h"
#include "MeshSelect.h"
#include "MeshTexturePaintingTool.h"
#include "MeshVertexPaintingTool.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "TexturePaintToolset.h"
#include "ToolContextInterfaces.h"
#include "LandscapeComponent.h"
#include "LandscapeNaniteComponent.h"
#include "LandscapeProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintMode)

#define LOCTEXT_NAMESPACE "MeshPaintMode"

FName UMeshPaintMode::MeshPaintMode_VertexColor = FName(TEXT("VertexColor"));
FName UMeshPaintMode::MeshPaintMode_VertexWeights = FName(TEXT("VertexWeights"));
FName UMeshPaintMode::MeshPaintMode_TextureColor = FName(TEXT("TextureColor"));
FName UMeshPaintMode::MeshPaintMode_TextureAsset = FName(TEXT("Texture"));

FString UMeshPaintMode::VertexSelectToolName = TEXT("VertexAdapterClickTool");
FString UMeshPaintMode::TextureColorSelectToolName = TEXT("TextureColorAdapterClickTool");
FString UMeshPaintMode::TextureAssetSelectToolName = TEXT("TextureAssetAdapterClickTool");

FString UMeshPaintMode::VertexColorPaintToolName = TEXT("VertexColorBrushTool");
FString UMeshPaintMode::VertexWeightPaintToolName = TEXT("VertexWeightBrushTool");
FString UMeshPaintMode::TextureColorPaintToolName = TEXT("TextureColorBrushTool");
FString UMeshPaintMode::TextureAssetPaintToolName = TEXT("TextureBrushTool");


UMeshPaintMode* UMeshPaintMode::GetMeshPaintMode()
{
	return Cast<UMeshPaintMode>(GLevelEditorModeTools().GetActiveScriptableMode("MeshPaintMode"));
}

FName UMeshPaintMode::GetValidPaletteName(const FName InName)
{
	if (InName == UMeshPaintMode::MeshPaintMode_VertexColor ||
		InName == UMeshPaintMode::MeshPaintMode_VertexWeights ||
		InName == UMeshPaintMode::MeshPaintMode_TextureColor ||
		InName == UMeshPaintMode::MeshPaintMode_TextureAsset)
	{
		return InName;
	}
	return UMeshPaintMode::MeshPaintMode_VertexColor;
}

template<typename T>
static T* GetTypedToolProperties()
{
	UMeshPaintMode* MeshPaintMode = UMeshPaintMode::GetMeshPaintMode();
	UInteractiveToolManager* ToolManager = MeshPaintMode != nullptr ? MeshPaintMode->GetToolManager() : nullptr;
	UInteractiveTool* Tool = ToolManager != nullptr ? ToolManager->GetActiveTool(EToolSide::Mouse) : nullptr;
	if (Tool != nullptr)
	{
		const TArray<UObject*> PropertyArray = Tool->GetToolProperties();
		for (UObject* Property : PropertyArray)
		{
			if (T* FoundProperty = Cast<T>(Property))
			{
				return FoundProperty;
			}
		}
	}
	return nullptr;
}

UMeshPaintingToolProperties* UMeshPaintMode::GetToolProperties()
{
	return GetTypedToolProperties<UMeshPaintingToolProperties>();
}

UMeshVertexPaintingToolProperties* UMeshPaintMode::GetVertexToolProperties()
{
	return GetTypedToolProperties<UMeshVertexPaintingToolProperties>();
}

UMeshVertexColorPaintingToolProperties* UMeshPaintMode::GetVertexColorToolProperties()
{
	return GetTypedToolProperties<UMeshVertexColorPaintingToolProperties>();
}

UMeshVertexWeightPaintingToolProperties* UMeshPaintMode::GetVertexWeightToolProperties()
{
	return GetTypedToolProperties<UMeshVertexWeightPaintingToolProperties>();
}

UMeshTexturePaintingToolProperties* UMeshPaintMode::GetTextureToolProperties()
{
	return GetTypedToolProperties<UMeshTexturePaintingToolProperties>();
}

UMeshTextureColorPaintingToolProperties* UMeshPaintMode::GetTextureColorToolProperties()
{
	return GetTypedToolProperties<UMeshTextureColorPaintingToolProperties>();
}

UMeshTextureAssetPaintingToolProperties* UMeshPaintMode::GetTextureAssetToolProperties()
{
	return GetTypedToolProperties<UMeshTextureAssetPaintingToolProperties>();
}


UMeshPaintMode::UMeshPaintMode()
	: Super()
{
	SettingsClass = UMeshPaintModeSettings::StaticClass();

	Info = FEditorModeInfo(
		FName(TEXT("MeshPaintMode")),
		LOCTEXT("ModeName", "Mesh Paint"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.MeshPaintMode", "LevelEditor.MeshPaintMode.Small"),
		true,
		600
	);
}

void UMeshPaintMode::Enter()
{
	Super::Enter();

	GEditor->OnEditorClose().AddUObject(this, &UMeshPaintMode::OnResetViewMode);
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UMeshPaintMode::OnObjectsReplaced);
	ModeSettings = Cast<UMeshPaintModeSettings>(SettingsObject);
	
	FMeshPaintEditorModeCommands ToolManagerCommands = FMeshPaintEditorModeCommands::Get();

	UVertexAdapterClickToolBuilder* VertexClickToolBuilder = NewObject<UVertexAdapterClickToolBuilder>(this);
	RegisterTool(ToolManagerCommands.SelectVertex, VertexSelectToolName, VertexClickToolBuilder);

	UTextureColorAdapterClickToolBuilder* TextureColorClickToolBuilder = NewObject<UTextureColorAdapterClickToolBuilder>(this);
	RegisterTool(ToolManagerCommands.SelectTextureColor, TextureColorSelectToolName, TextureColorClickToolBuilder);

	UTextureAssetAdapterClickToolBuilder* TextureAssetClickToolBuilder = NewObject<UTextureAssetAdapterClickToolBuilder>(this);
	RegisterTool(ToolManagerCommands.SelectTextureAsset, TextureAssetSelectToolName, TextureAssetClickToolBuilder);

	UMeshVertexColorPaintingToolBuilder* MeshColorPaintingToolBuilder = NewObject<UMeshVertexColorPaintingToolBuilder>(this);
	RegisterTool(ToolManagerCommands.PaintVertexColor, VertexColorPaintToolName, MeshColorPaintingToolBuilder);

	UMeshVertexWeightPaintingToolBuilder* WeightPaintingToolBuilder = NewObject<UMeshVertexWeightPaintingToolBuilder>(this);
	RegisterTool(ToolManagerCommands.PaintVertexWeight, VertexWeightPaintToolName, WeightPaintingToolBuilder);

	UMeshTextureColorPaintingToolBuilder* MeshTextureColorPaintingToolBuilder = NewObject<UMeshTextureColorPaintingToolBuilder>(this);
	RegisterTool(ToolManagerCommands.PaintTextureColor, TextureColorPaintToolName, MeshTextureColorPaintingToolBuilder);

	UMeshTextureAssetPaintingToolBuilder* TextureAssetPaintingToolBuilder = NewObject<UMeshTextureAssetPaintingToolBuilder>(this);
	RegisterTool(ToolManagerCommands.PaintTextureAsset, TextureAssetPaintToolName, TextureAssetPaintingToolBuilder);
	
	UpdateSelectedMeshes();

	// Toolkit
	PaletteChangedHandle = Toolkit->OnPaletteChanged().AddUObject(this, &UMeshPaintMode::UpdateOnPaletteChange);

	// disable tool change tracking to activate default tool
	GetToolManager()->ConfigureChangeTrackingMode(EToolChangeTrackingMode::NoChangeTracking);
	Toolkit->SetCurrentPalette(GetValidPaletteName(ModeSettings->DefaultPalette));
	// switch back to full undo / redo tracking mode here if that is behavior we want
	//GetToolManager()->ConfigureChangeTrackingMode(EToolChangeTrackingMode::FullUndoRedo);

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(FName(TEXT("LevelEditor")));
	LevelEditor.OnRedrawLevelEditingViewports().AddUObject(this, &UMeshPaintMode::UpdateOnMaterialChange);

	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddUObject(this, &UMeshPaintMode::UpdateOnPostCompile);

	// some global cvars can affect whether painting is valid (nanite on/off etc)
	CVarDelegateHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateLambda([this]{ bRecacheValidForPaint = true; }));
}

void UMeshPaintMode::Exit()
{
	ModeSettings->DefaultPalette = Toolkit->GetCurrentPalette();

	Toolkit->OnPaletteChanged().Remove(PaletteChangedHandle);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	GEditor->OnEditorClose().RemoveAll(this);
	OnResetViewMode();

	const FMeshPaintEditorModeCommands& Commands = FMeshPaintEditorModeCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	for (auto It : Commands.Commands)
	{
		for (const TSharedPtr<const FUICommandInfo> Action : It.Value)
		{
			CommandList->UnmapAction(Action);
		}
	}

	Super::Exit();

	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->ResetState();

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(FName(TEXT("LevelEditor")));
	LevelEditor.OnRedrawLevelEditingViewports().RemoveAll(this);

	FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);

	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarDelegateHandle);
	CVarDelegateHandle = {};
}

void UMeshPaintMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FMeshPaintModeToolkit);
}

void UMeshPaintMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (bRecacheDataSizes)
	{
		UpdateCachedDataSizes();
	}

	if (bRecacheValidForPaint)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->UpdatePaintSupportState();
		bRecacheValidForPaint = false;
	}

	// Close the active paint tool if selection (or other state) changes mean that it's not longer valid to paint.
	// For example if the selected component or it's materials no longer supports texture painting.
	EndPaintToolIfNoLongerValid();

	// Make sure that correct tab is visible for the current tool.
	// Note that currently Color and Weight mode share the same Select tool.
	UInteractiveTool const* ActiveTool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
	const FString ActiveToolName = GetToolManager()->GetActiveToolName(EToolSide::Mouse);

	FName ActiveTab = Toolkit->GetCurrentPalette();
	FName TargetTab = ActiveTab;
	
	if (ActiveToolName == VertexColorPaintToolName)
	{
		TargetTab = MeshPaintMode_VertexColor;
	}
	else if (ActiveToolName == VertexWeightPaintToolName)
	{
		TargetTab = MeshPaintMode_VertexWeights;
	}
	else if (ActiveToolName == TextureColorPaintToolName || ActiveToolName == TextureColorSelectToolName)
	{
		TargetTab = MeshPaintMode_TextureColor;
	}
	else if (ActiveToolName == TextureAssetPaintToolName || ActiveToolName == TextureAssetSelectToolName)
	{
		TargetTab = MeshPaintMode_TextureAsset;
	}

	EMeshPaintActiveMode CurrentActiveMode = EMeshPaintActiveMode::VertexColor;

	if (TargetTab == MeshPaintMode_VertexColor)
	{
		CurrentActiveMode = EMeshPaintActiveMode::VertexColor;
	}
	else if (TargetTab == MeshPaintMode_VertexWeights)
	{
		CurrentActiveMode = EMeshPaintActiveMode::VertexWeights;
	}
	else if (TargetTab == MeshPaintMode_TextureColor)
	{
		CurrentActiveMode = EMeshPaintActiveMode::TextureColor;
	}
	else if (TargetTab == MeshPaintMode_TextureAsset)
	{
		CurrentActiveMode = EMeshPaintActiveMode::Texture;
	}

	if (TargetTab != ActiveTab || ActiveTool == nullptr)
	{
		Toolkit->SetCurrentPalette(TargetTab);
	}

	if (ViewportClient->IsPerspective())
	{
		// Make sure perspective viewports are still set to real-time
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SetRealtimeViewport(ViewportClient, true);

		// Set viewport show flags
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SetViewportColorMode(CurrentActiveMode, ModeSettings->ColorViewMode, ViewportClient, ActiveTool);
	}
}

bool UMeshPaintMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	return true;
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> UMeshPaintMode::GetModeCommands() const
{
	return FMeshPaintEditorModeCommands::GetCommands();
}

void UMeshPaintMode::BindCommands()
{
	const FMeshPaintEditorModeCommands& Commands = FMeshPaintEditorModeCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(Commands.SwapColor, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::SwapColors),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanSwapColors)));

 	CommandList->MapAction(Commands.FillVertex,	FUIAction(
 		FExecuteAction::CreateUObject(this, &UMeshPaintMode::FillVertexColors),
 		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanFillVertexColors)));

 	CommandList->MapAction(Commands.FillTexture, FUIAction(
 		FExecuteAction::CreateUObject(this, &UMeshPaintMode::FillTexture),
 		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanFillTexture)));

 	CommandList->MapAction(Commands.PropagateMesh, FUIAction(
 		FExecuteAction::CreateUObject(this, &UMeshPaintMode::PropagateVertexColorsToMesh),
 		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanPropagateVertexColorsToMesh)));

	CommandList->MapAction(Commands.PropagateLODs, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::PropagateVertexColorsToLODs),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanPropagateVertexColorsToLODs)));

	CommandList->MapAction(Commands.SaveVertex,	FUIAction(
			FExecuteAction::CreateUObject(this, &UMeshPaintMode::SaveVertexColorsToAssets),
			FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanSaveVertexColorsToAssets)));

 	CommandList->MapAction(Commands.SaveTexture, FUIAction(
 		FExecuteAction::CreateUObject(this, &UMeshPaintMode::SaveTexturePackages),
 		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanSaveTexturePackages)));

	CommandList->MapAction(Commands.Add, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::AddMeshPaintTextures),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanAddMeshPaintTextures)));

	CommandList->MapAction(Commands.RemoveVertex, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::RemoveInstanceVertexColors),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanRemoveInstanceVertexColors)));

	CommandList->MapAction(Commands.RemoveTexture, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::RemoveMeshPaintTexture),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanRemoveMeshPaintTextures)));

 	CommandList->MapAction(Commands.Copy, FUIAction(
 		FExecuteAction::CreateUObject(this, &UMeshPaintMode::Copy),
 		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanCopy)));

	CommandList->MapAction(Commands.Paste, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::Paste),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanPaste)));

	CommandList->MapAction(Commands.Import, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ImportVertexColorsFromFile),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanImportVertexColorsFromFile)));

	CommandList->MapAction(Commands.GetTextureColors, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ImportVertexColorsFromMeshPaintTexture),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanImportVertexColorsFromMeshPaintTexture)));

	CommandList->MapAction(Commands.GetVertexColors, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ImportMeshPaintTextureFromVertexColors),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanImportMeshPaintTextureFromVertexColors)));

	CommandList->MapAction(Commands.FixVertex, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::FixVertexColors),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanFixVertexColors)));

	CommandList->MapAction(Commands.FixTexture, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::FixTextureColors),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanFixTextureColors)));

	CommandList->MapAction(Commands.PreviousLOD, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::CycleMeshLODs, -1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanCycleMeshLODs)));

	CommandList->MapAction(Commands.NextLOD, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::CycleMeshLODs, 1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanCycleMeshLODs)));

	CommandList->MapAction(Commands.PreviousTexture, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::CycleTextures, -1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanCycleTextures)));

	CommandList->MapAction(Commands.NextTexture, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::CycleTextures, 1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanCycleTextures)));

	CommandList->MapAction(Commands.IncreaseBrushRadius, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushRadius, 1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));
	
	CommandList->MapAction(Commands.DecreaseBrushRadius, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushRadius, -1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));

	CommandList->MapAction(Commands.IncreaseBrushStrength, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushStrength, 1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));
	
	CommandList->MapAction(Commands.DecreaseBrushStrength, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushStrength, -1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));

	CommandList->MapAction(Commands.IncreaseBrushFalloff, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushFalloff, 1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));

	CommandList->MapAction(Commands.DecreaseBrushFalloff, FUIAction(
		FExecuteAction::CreateUObject(this, &UMeshPaintMode::ChangeBrushFalloff, -1),
		FCanExecuteAction::CreateUObject(this, &UMeshPaintMode::CanChangeBrush),
		EUIActionRepeatMode::RepeatEnabled));
}

void UMeshPaintMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FMeshPaintingToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), false);

	if (UMeshVertexPaintingTool* VertexPaintingTool = Cast<UMeshVertexPaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		VertexPaintingTool->OnPaintingFinished().BindUObject(this, &UMeshPaintMode::OnVertexPaintFinished);
	}

	if (UMeshTextureColorPaintingTool* TextureColorPaintingTool = Cast<UMeshTextureColorPaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		TextureColorPaintingTool->OnPaintingFinished().BindUObject(this, &UMeshPaintMode::OnTextureColorVertexPaintFinished);
	}
}

void UMeshPaintMode::OnVertexPaintFinished()
{
	if (UMeshVertexPaintingToolProperties* VertexPaintingToolProperties = UMeshPaintMode::GetVertexToolProperties())
	{
		if (!VertexPaintingToolProperties->bPaintOnSpecificLOD)
		{
			PropagateVertexColorsToLODs();
		}
		else
		{
			if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
			{
				bRecacheDataSizes = true;
				MeshPaintingSubsystem->Refresh();
			}
		}
	}
}

void UMeshPaintMode::OnTextureColorVertexPaintFinished(UMeshComponent* MeshComponent)
{
	if (UMeshTextureColorPaintingToolProperties* TexturePaintingToolProperties = UMeshPaintMode::GetTextureColorToolProperties())
	{
		if (TexturePaintingToolProperties->bPropagateToVertexColor)
		{
			GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->ImportVertexColorsFromMeshPaintTexture(MeshComponent);
		}
	}
}

void UMeshPaintMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FMeshPaintingToolActionCommands::UpdateToolCommandBinding(Tool, Toolkit->GetToolkitCommands(), true);
	// First update your bindings, then call the base behavior
	Super::OnToolEnded(Manager, Tool);
}

void UMeshPaintMode::ActorSelectionChangeNotify()
{
	UpdateSelectedMeshes();
}

void UMeshPaintMode::ElementSelectionChangeNotify()
{
	UpdateSelectedMeshes();
}

void UMeshPaintMode::ActorPropChangeNotify()
{
	// Setting change on selected components can change whether they are valid for painting.
	bRecacheValidForPaint = true;
}

void UMeshPaintMode::UpdateSelectedMeshes()
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->ResetState();
		TArray<UMeshComponent*> CurrentMeshComponents = GetSelectedComponents<UMeshComponent>();
		MeshPaintingSubsystem->AddSelectedMeshComponents(CurrentMeshComponents);
		MeshPaintingSubsystem->bNeedsRecache = true;
	}
	
	bRecacheDataSizes = true;
	bRecacheValidForPaint = true;
}

void UMeshPaintMode::EndPaintToolIfNoLongerValid()
{
	bool bInvalidTool = false;

	UInteractiveToolManager* ToolManager = GetToolManager();
	UInteractiveTool const* Tool = (ToolManager == nullptr) ? nullptr : ToolManager->GetActiveTool(EToolSide::Mouse);
	if (Tool != nullptr)
	{
		UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();

		if (Tool->IsA<UMeshVertexPaintingTool>())
		{
			bInvalidTool = !MeshPaintingSubsystem->GetSelectionSupportsVertexPaint();
		}
		else if (Tool->IsA<UMeshTextureColorPaintingTool>())
		{
			bInvalidTool = !MeshPaintingSubsystem->GetSelectionSupportsTextureColorPaint();
		}
		else if (Tool->IsA<UMeshTextureAssetPaintingTool>())
		{
			bInvalidTool = !MeshPaintingSubsystem->GetSelectionSupportsTextureAssetPaint();
		}
	}

	if (bInvalidTool)
	{
		GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept);
		ActivateDefaultTool();
	}
}

void UMeshPaintMode::UpdateOnMaterialChange(bool bInvalidateHitProxies)
{
	// Need to recheck whether the current material supports texture paint.
	bRecacheValidForPaint = true;
}

void UMeshPaintMode::UpdateOnPostCompile(const TArray<FAssetCompileData>& Data)
{
	// Can happen after add/remove nanite data.
	bRecacheValidForPaint = true;
}

void UMeshPaintMode::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstanceMap)
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->ClearSelectedMeshComponents();
		MeshPaintingSubsystem->Refresh();
		UpdateSelectedMeshes();
	}
}

void UMeshPaintMode::FillVertexColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionFillInstColors", "Filling Per-Instance Vertex Colors"));
	const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	static const bool bConvertSRGB = false;
	FColor FillColor = FColor::White;
	FColor MaskColor = FColor::White;

	if (GetToolManager()->GetActiveTool(EToolSide::Mouse)->IsA<UMeshVertexWeightPaintingTool>())
	{
		FillColor = MeshPaintingSubsystem->GenerateColorForTextureWeight((int32)GetVertexWeightToolProperties()->TextureWeightType, (int32)GetVertexWeightToolProperties()->PaintTextureWeightIndex).ToFColor(bConvertSRGB);
	}
	else if (UMeshVertexColorPaintingToolProperties* ColorProperties = GetVertexColorToolProperties())
	{
		FillColor = ColorProperties->PaintColor.ToFColor(bConvertSRGB);
		MaskColor.R = ColorProperties->bWriteRed ? 255 : 0;
		MaskColor.G = ColorProperties->bWriteGreen ? 255 : 0;
		MaskColor.B = ColorProperties->bWriteBlue ? 255 : 0;
		MaskColor.A = ColorProperties->bWriteAlpha ? 255 : 0;
	}

	TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
	/** Fill each mesh component with the given vertex color */
	for (UMeshComponent* Component : MeshComponents)
	{
		checkf(Component != nullptr, TEXT("Invalid Mesh Component"));
		Component->Modify();
		ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component);

		TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(Component);
		if (MeshAdapter)
		{
			MeshAdapter->PreEdit();
		}

		UMeshVertexPaintingToolProperties* VertexProperties = UMeshPaintMode::GetVertexToolProperties();
		const bool bPaintOnSpecificLOD = VertexProperties ? VertexProperties->bPaintOnSpecificLOD : false;

		if (Component->IsA<UStaticMeshComponent>())
		{
			MeshPaintingSubsystem->FillStaticMeshVertexColors(Cast<UStaticMeshComponent>(Component), bPaintOnSpecificLOD ? VertexProperties->LODIndex : -1, FillColor, MaskColor);
		}
		else if (Component->IsA<USkeletalMeshComponent>())
		{
			GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->FillSkeletalMeshVertexColors(Cast<USkeletalMeshComponent>(Component), bPaintOnSpecificLOD ? VertexProperties->LODIndex : -1, FillColor, MaskColor);
		}
		else if (MeshAdapter) 			// We don't have a custom fill function for this type of component; try to go through the adapter.
		{
			TArray<uint32> MeshIndices = MeshAdapter->GetMeshIndices();
			UMeshPaintingSubsystem* PaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
			for (uint32 VID : MeshIndices)
			{
				FColor Color;
				MeshAdapter->GetVertexColor((int32)VID, Color);
				PaintingSubsystem->ApplyFillWithMask(Color, MaskColor, FillColor);
				MeshAdapter->SetVertexColor((int32)VID, Color);
			}
		}

		if (MeshAdapter)
		{
			MeshAdapter->PostEdit();
		}
	}
}


void UMeshPaintMode::PropagateVertexColorsToMesh()
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	FSuppressableWarningDialog::FSetupInfo SetupInfo(LOCTEXT("PushInstanceVertexColorsPrompt_Message", "This operation copies vertex colors from LOD 0 of the selected instance to all LODs of the source asset, overwriting any existing vertex colors.\n\nThis change will also propagate to all other instances of the same asset that do not have custom vertex colors."),
		LOCTEXT("PushInstanceVertexColorsPrompt_Title", "Warning: Overwriting Vertex Colors on Source Asset"), "Warning_PushInstanceVertexColorsPrompt");

	SetupInfo.ConfirmText = LOCTEXT("PushInstanceVertexColorsPrompt_ConfirmText", "Overwrite");
	SetupInfo.CancelText = LOCTEXT("PushInstanceVertexColorsPrompt_CancelText", "Cancel");
	SetupInfo.CheckBoxText = LOCTEXT("PushInstanceVertexColorsPrompt_CheckBoxText", "Always overwrite source asset without prompting");

	FSuppressableWarningDialog VertexColorCopyWarning(SetupInfo);

	// Prompt the user to see if they really want to push the vert colors to the source mesh and to explain
	// the ramifications of doing so. This uses a suppressible dialog so that the user has the choice to always ignore the warning.
	if (VertexColorCopyWarning.ShowModal() != FSuppressableWarningDialog::Cancel)
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionPropogateColors", "Propagating Vertex Colors To Source Meshes"));
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->PropagateVertexColors(StaticMeshComponents);
	}
}

bool UMeshPaintMode::CanPropagateVertexColorsToMesh() const
{
	// Check whether or not our selected Static Mesh Components contain instance based vertex colors (only these can be propagated to the base mesh)
	int32 NumInstanceVertexColorBytes = 0;

	TArray<UStaticMesh*> StaticMeshes;
	TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	return GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->CanPropagateVertexColors(StaticMeshComponents, StaticMeshes, NumInstanceVertexColorBytes);
}

void UMeshPaintMode::ImportVertexColorsFromFile()
{
	const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
	if (MeshComponents.Num() == 1)
	{
		/** Import vertex color to single selected mesh component */
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionImportColors", "Importing Vertex Colors From Texture"));
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->ImportVertexColorsFromTexture(MeshComponents[0]);
	}

	bRecacheDataSizes = true;
}

void UMeshPaintMode::SaveVertexColorsToAssets()
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	const TArray<USkeletalMeshComponent*> SkeletalMeshComponents = GetSelectedComponents<USkeletalMeshComponent>();

	/** Try and save outstanding dirty packages for currently selected mesh components */
	TArray<UObject*> ObjectsToSave;
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh())
		{
			ObjectsToSave.Add(StaticMeshComponent->GetStaticMesh());
		}
	}

	for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
	{
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			ObjectsToSave.Add(SkeletalMeshComponent->GetSkeletalMeshAsset());
		}
	}

	if (ObjectsToSave.Num() > 0)
	{
		UPackageTools::SavePackagesForObjects(ObjectsToSave);
	}
}

bool UMeshPaintMode::CanSaveVertexColorsToAssets() const
{
	// Check whether or not any of our selected mesh components contain mesh objects which require saving
	TArray<UMeshComponent*> Components = GetSelectedComponents<UMeshComponent>();

	bool bValid = false;

	for (UMeshComponent* Component : Components)
	{
		UObject* Object = nullptr;
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			Object = StaticMeshComponent->GetStaticMesh();
		}
		else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
		{
			Object = SkeletalMeshComponent->GetSkeletalMeshAsset();
		}

		if (Object != nullptr && Object->GetOutermost()->IsDirty())
		{
			bValid = true;
			break;
		}
	}

	return bValid;
}

bool UMeshPaintMode::CanRemoveInstanceVertexColors() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	int32 PaintingMeshLODIndex = 0;
	if (UMeshVertexPaintingToolProperties* VertexProperties = UMeshPaintMode::GetVertexToolProperties())
	{
		PaintingMeshLODIndex = VertexProperties->bPaintOnSpecificLOD ? VertexProperties->LODIndex : 0;
	}
	int32 NumValidMeshes = 0;
	// Retrieve per instance vertex color information (only valid if the component contains actual instance vertex colors)
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component != nullptr && Component->GetStaticMesh() != nullptr && Component->GetStaticMesh()->GetNumLODs() > (int32)PaintingMeshLODIndex)
		{
			uint32 BufferSize = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetVertexColorBufferSize(Component, PaintingMeshLODIndex, true);

			if (BufferSize > 0)
			{
				++NumValidMeshes;
			}
		}
	}

	return (NumValidMeshes != 0);
}

bool UMeshPaintMode::CanPasteInstanceVertexColors() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	const TArray<FPerComponentVertexColorData> CopiedColorsByComponent = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetCopiedColorsByComponent();
	return GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->CanPasteInstanceVertexColors(StaticMeshComponents, CopiedColorsByComponent);
}

bool UMeshPaintMode::CanCopyInstanceVertexColors() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	int32 PaintingMeshLODIndex = 0;
	if (UMeshVertexPaintingToolProperties* VertexProperties = UMeshPaintMode::GetVertexToolProperties())
	{
		PaintingMeshLODIndex = VertexProperties->bPaintOnSpecificLOD ? VertexProperties->LODIndex : 0;
	}

	return GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->CanCopyInstanceVertexColors(StaticMeshComponents, PaintingMeshLODIndex);

}

bool UMeshPaintMode::CanPropagateVertexColorsToLODs() const
{
	bool bPaintOnSpecificLOD = false;
	if (UMeshVertexPaintingToolProperties* VertexProperties = UMeshPaintMode::GetVertexToolProperties())
	{
		bPaintOnSpecificLOD = VertexProperties ? VertexProperties->bPaintOnSpecificLOD : false;
	}
	// Can propagate when the mesh contains per-lod vertex colors or when we are not painting to a specific lod
	const bool bSelectionContainsPerLODColors = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SelectionContainsPerLODColors();
	return bSelectionContainsPerLODColors || !bPaintOnSpecificLOD;
}

void UMeshPaintMode::CopyInstanceVertexColors()
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	TArray<FPerComponentVertexColorData> CopiedColorsByComponent;
	GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->CopyVertexColors(StaticMeshComponents, CopiedColorsByComponent);
	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SetCopiedColorsByComponent(CopiedColorsByComponent);
}

void UMeshPaintMode::PasteInstanceVertexColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionPasteInstColors", "Pasting Per-Instance Vertex Colors"));
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	TArray<FPerComponentVertexColorData> CopiedColorsByComponent = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetCopiedColorsByComponent();
	GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->PasteVertexColors(StaticMeshComponents, CopiedColorsByComponent);
	
	bRecacheDataSizes = true;
}

void UMeshPaintMode::FixVertexColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionFixInstColors", "Fixing Per-Instance Vertex Colors"));
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		Component->FixupOverrideColorsIfNecessary();
	}

	bRecacheDataSizes = true;
}

bool UMeshPaintMode::CanFixVertexColors() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	bool bAnyMeshNeedsFixing = false;
	/** Check if there are any static mesh components which require fixing */
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		bAnyMeshNeedsFixing |= Component->RequiresOverrideVertexColorsFixup();
	}

	return bAnyMeshNeedsFixing;
}

void UMeshPaintMode::RemoveInstanceVertexColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionRemoveInstColors", "Removing Per-Instance Vertex Colors"));
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->RemoveComponentInstanceVertexColors(Component);
	}

	bRecacheDataSizes = true;
}


void UMeshPaintMode::PropagateVertexColorsToLODs()
{
	//Only show the lost data warning if there is actually some data to lose
	bool bAbortChange = false;
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();
	const bool bSelectionContainsPerLODColors = MeshPaintingSubsystem->SelectionContainsPerLODColors();
	if (bSelectionContainsPerLODColors)
	{
		//Warn the user they will lose custom painting data
		FSuppressableWarningDialog::FSetupInfo SetupInfo(LOCTEXT("LooseLowersLODsVertexColorsPrompt_Message", "This operation copies vertex colors from LOD 0 to all other LODs in this instance, overwriting any existing vertex colors.\n\nAt least one LOD has custom vertex colors that will be lost."),
			LOCTEXT("LooseLowersLODsVertexColorsPrompt_Title", "Warning: Overwriting Vertex Colors on LODs"), "Warning_LooseLowersLODsVertexColorsPrompt");

		SetupInfo.ConfirmText = LOCTEXT("LooseLowersLODsVertexColorsPrompt_ConfirmText", "Overwrite");
		SetupInfo.CancelText = LOCTEXT("LooseLowersLODsVertexColorsPrompt_CancelText", "Cancel");
		SetupInfo.CheckBoxText = LOCTEXT("LooseLowersLODsVertexColorsPrompt_CheckBoxText", "Always overwrite LODs without prompting");

		FSuppressableWarningDialog LooseLowersLODsVertexColorsWarning(SetupInfo);

		// Prompt the user to see if they really want to propagate the base lod vert colors to the lowers LODs.
		if (LooseLowersLODsVertexColorsWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
		{
			bAbortChange = true;
		}
		else
		{
			// Reset the state flag as we'll be removing all per-lod colors 
			MeshPaintingSubsystem->ClearSelectionLODColors();
			GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->RemovePerLODColors(PaintableComponents);
		}
	}

	//The user cancel the change, avoid changing the value
	if (bAbortChange)
	{
		return;
	}

	for (UMeshComponent* SelectedComponent : PaintableComponents)
	{
		if (SelectedComponent)
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(SelectedComponent);
			GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->ApplyVertexColorsToAllLODs(*MeshAdapter, SelectedComponent);
			FComponentReregisterContext ReregisterContext(SelectedComponent);
		}
	}

	bRecacheDataSizes = true;
	
	MeshPaintingSubsystem->Refresh();
}

template<typename ComponentClass>
TArray<ComponentClass*> UMeshPaintMode::GetSelectedComponents() const
{
	FToolBuilderState SelectionState;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentSelectionState(SelectionState);

	TArray<ComponentClass*> Components;
	for (int32 SelectionIndex = 0; SelectionIndex < SelectionState.SelectedComponents.Num(); ++SelectionIndex)
	{
		ComponentClass* SelectedComponent = Cast<ComponentClass>(SelectionState.SelectedComponents[SelectionIndex]);
		if (SelectedComponent)
		{
			Components.AddUnique(SelectedComponent);
		}
	}

	if (Components.Num() == 0)
	{
		for (AActor* SelectedActor : SelectionState.SelectedActors)
		{
			if (SelectedActor)
			{
				// Code inside branch is only useful and compiles when ULandscapeNaniteComponent is derived from ComponentClass.
				if constexpr (TIsDerivedFrom<ULandscapeNaniteComponent, ComponentClass>::Value)
				{
					// Selected landscape actors may contain many mesh components that we are not interested in (landscape grass, spline etc).
					// To avoid performance/selection issues we assume that we only want to paint on the Nanite Landscape Components.
					if (SelectedActor->IsA<ALandscapeProxy>())
					{
						TInlineComponentArray<ULandscapeNaniteComponent*> ActorComponents;
						SelectedActor->GetComponents(ActorComponents);
						for (ULandscapeNaniteComponent* Component : ActorComponents)
						{
							Components.AddUnique(Component);
						}
						
						continue;
					}
				}

				TInlineComponentArray<ComponentClass*> ActorComponents;
				SelectedActor->GetComponents(ActorComponents);
				for (ComponentClass* Component : ActorComponents)
				{
					Components.AddUnique(Component);
				}
			}
		}
	}

	return Components;
}

template TArray<UStaticMeshComponent*> UMeshPaintMode::GetSelectedComponents<UStaticMeshComponent>() const;
template TArray<USkeletalMeshComponent*> UMeshPaintMode::GetSelectedComponents<USkeletalMeshComponent>() const;
template TArray<UMeshComponent*> UMeshPaintMode::GetSelectedComponents<UMeshComponent>() const;
template TArray<UGeometryCollectionComponent*> UMeshPaintMode::GetSelectedComponents<UGeometryCollectionComponent>() const;

void UMeshPaintMode::UpdateCachedDataSizes()
{
	CachedVertexDataSize = 0;
	CachedMeshPaintTextureResourceSize = 0;

	const bool bInstance = true;
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
		for (UMeshComponent* MeshComponent : MeshComponents)
		{
			int32 NumLODs = MeshPaintingSubsystem->GetNumberOfLODs(MeshComponent);
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				CachedVertexDataSize += MeshPaintingSubsystem->GetVertexColorBufferSize(MeshComponent, LODIndex, bInstance);
			}

			CachedMeshPaintTextureResourceSize  += MeshPaintingSubsystem->GetMeshPaintTextureResourceSize(MeshComponent);
		}
	}

	bRecacheDataSizes = false;
}

void UMeshPaintMode::CycleMeshLODs(int32 Direction)
{
	if (UMeshVertexPaintingTool* VertexPaintingTool = Cast<UMeshVertexPaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_CycleLOD", "Changed Current LOD"));
		VertexPaintingTool->CycleMeshLODs(Direction);
	}
}

void UMeshPaintMode::CycleTextures(int32 Direction)
{
	if (UMeshTextureAssetPaintingTool* TexturePaintingTool = Cast<UMeshTextureAssetPaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_CycleTexture", "Changed Current Texture"));
		TexturePaintingTool->CycleTextures(Direction);
	}
}

bool UMeshPaintMode::CanCycleTextures() const
{
	UMeshTextureAssetPaintingTool* TexturePaintingTool = Cast<UMeshTextureAssetPaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left));
	return TexturePaintingTool != nullptr;
}

void UMeshPaintMode::ChangeBrushRadius(int32 Direction)
{
	if (UBaseBrushTool* Tool = Cast<UBaseBrushTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_ChangeBrushRadius", "Changed Brush Radius"));
		if (Direction > 0)
		{
			Tool->IncreaseBrushSizeAction();
		}
		else
		{
			Tool->DecreaseBrushSizeAction();
		}
	}
}

void UMeshPaintMode::ChangeBrushStrength(int32 Direction)
{
	if (UBaseBrushTool* Tool = Cast<UBaseBrushTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_ChangeBrushStrength", "Changed Brush Strength"));
		if (Direction > 0)
		{
			Tool->IncreaseBrushStrengthAction();
		}
		else
		{
			Tool->DecreaseBrushStrengthAction();
		}
	}
}

void UMeshPaintMode::ChangeBrushFalloff(int32 Direction)
{
	if (UBaseBrushTool* Tool = Cast<UBaseBrushTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_ChangeBrushFalloff", "Changed Brush Falloff"));
		if (Direction > 0)
		{
			Tool->IncreaseBrushFalloffAction();
		}
		else
		{
			Tool->DecreaseBrushFalloffAction();
		}
	}
}

bool UMeshPaintMode::CanChangeBrush() const
{
	UBaseBrushTool* Tool = Cast<UBaseBrushTool>(GetToolManager()->GetActiveTool(EToolSide::Left));
	return Tool != nullptr;
}

void UMeshPaintMode::ActivateDefaultTool()
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	if (PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		GetInteractiveToolsContext()->StartTool(VertexSelectToolName);
	}
	if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		GetInteractiveToolsContext()->StartTool(TextureColorSelectToolName);
	}
	if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureAsset)
	{
		GetInteractiveToolsContext()->StartTool(TextureAssetSelectToolName);
	}
}

void UMeshPaintMode::UpdateOnPaletteChange(FName NewPaletteName)
{
	UpdateSelectedMeshes();

	FString SwitchToToolPaint;
	FString SwitchToToolSelect;
	if (NewPaletteName == UMeshPaintMode::MeshPaintMode_VertexColor)
	{
		SwitchToToolPaint = VertexColorPaintToolName;
		SwitchToToolSelect = VertexSelectToolName;
	}
	else if (NewPaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		SwitchToToolPaint = VertexWeightPaintToolName;
		SwitchToToolSelect = VertexSelectToolName;
	}
	else if (NewPaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		SwitchToToolPaint = TextureColorPaintToolName;
		SwitchToToolSelect = TextureColorSelectToolName;
	}
	else if (NewPaletteName == UMeshPaintMode::MeshPaintMode_TextureAsset)
	{
		SwitchToToolPaint = TextureAssetPaintToolName;
		SwitchToToolSelect = TextureAssetSelectToolName;
	}

	if (!SwitchToToolPaint.IsEmpty())
	{
		// Figure out which tool we would like to be in based on currently-active tool
		const FString ActiveTool = GetToolManager()->GetActiveToolName(EToolSide::Mouse);
		const bool bInAnyPaintTool = (ActiveTool == VertexColorPaintToolName || ActiveTool == VertexWeightPaintToolName || ActiveTool == TextureColorPaintToolName || ActiveTool == TextureAssetPaintToolName);
		const bool bUsePaintTool = bInAnyPaintTool && GetInteractiveToolsContext()->CanStartTool(SwitchToToolPaint);
		const FString SwitchToTool = bUsePaintTool ? SwitchToToolPaint : SwitchToToolSelect;

		// Change to new tool if it is different
		if (SwitchToTool != ActiveTool)
		{
			GetInteractiveToolsContext()->StartTool(SwitchToTool);
		}
	}
}

void UMeshPaintMode::FillTexture()
{
	if (UMeshTexturePaintingTool* TexturePaintingTool = Cast<UMeshTexturePaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		TexturePaintingTool->FloodCurrentPaintTexture();
	}
}

void UMeshPaintMode::OnResetViewMode()
{
	// Reset viewport color mode and realtime override for all active viewports
	for (FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		if (!ViewportClient || ViewportClient->GetModeTools() != GetModeManager())
		{
			continue;
		}

		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SetViewportColorMode(EMeshPaintActiveMode::VertexColor, EMeshPaintDataColorViewMode::Normal, ViewportClient, nullptr);
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SetRealtimeViewport(ViewportClient, false);
	}
}

bool UMeshPaintMode::IsInSelectTool() const
{
	FString ActiveTool = GetToolManager()->GetActiveToolName(EToolSide::Mouse);
	return ActiveTool == VertexSelectToolName || ActiveTool == TextureColorSelectToolName || ActiveTool == TextureAssetSelectToolName;
}

bool UMeshPaintMode::IsInPaintTool() const
{
	FString ActiveTool = GetToolManager()->GetActiveToolName(EToolSide::Mouse);
	return ActiveTool == VertexColorPaintToolName || ActiveTool == VertexWeightPaintToolName || ActiveTool == TextureColorPaintToolName || ActiveTool == TextureAssetPaintToolName;
}

void UMeshPaintMode::SwapColors()
{
	GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->SwapColors();
}

bool UMeshPaintMode::CanSwapColors() const
{
	return IsInPaintTool();
}

bool UMeshPaintMode::CanFillVertexColors() const
{
	return IsInPaintTool();
}

bool UMeshPaintMode::CanFillTexture() const
{
	return IsInPaintTool();
}

void UMeshPaintMode::SaveTexturePackages()
{
	TArray<UObject*> TexturesToSave;
	if (UMeshTexturePaintingTool* TexturePaintingTool = Cast<UMeshTexturePaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		TexturePaintingTool->GetModifiedTexturesToSave(TexturesToSave);
	}
	if (TexturesToSave.Num())
	{
		UPackageTools::SavePackagesForObjects(TexturesToSave);
	}
}

bool UMeshPaintMode::CanSaveTexturePackages() const
{
	TArray<UObject*> TexturesToSave;
	if (UMeshTexturePaintingTool* TexturePaintingTool = Cast<UMeshTexturePaintingTool>(GetToolManager()->GetActiveTool(EToolSide::Left)))
	{
		TexturePaintingTool->GetModifiedTexturesToSave(TexturesToSave);
	}
	return TexturesToSave.Num() > 0;
}

void UMeshPaintMode::AddMeshPaintTextures()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionAddMeshPaintTexture", "Creating Mesh Paint Texture"));
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->CreateComponentMeshPaintTexture(Component);
	}

	bRecacheDataSizes = true;
	bRecacheValidForPaint = true;
}

bool UMeshPaintMode::CanAddMeshPaintTextures() const
{
	if (IsInSelectTool())
	{
		const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
		for (UStaticMeshComponent* Component : StaticMeshComponents)
		{
			if (Component->GetMeshPaintTexture() == nullptr && Component->CanMeshPaintTextureColors())
			{
				return true;
			}
		}
	}
	return false;
}

void UMeshPaintMode::RemoveMeshPaintTexture()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionRemoveMeshPaintTexture", "Removing Mesh Paint Texture"));
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->RemoveComponentMeshPaintTexture(Component);
	}

	bRecacheDataSizes = true;
	bRecacheValidForPaint = true;
}

bool UMeshPaintMode::CanRemoveMeshPaintTextures() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component->GetMeshPaintTexture() != nullptr)
		{
			return true;
		}
	}
	return false;
}

void UMeshPaintMode::CopyMeshPaintTexture()
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	if (StaticMeshComponents.Num() > 0)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SetCopiedTexture(StaticMeshComponents[0]->GetMeshPaintTexture());
	}
}

bool UMeshPaintMode::CanCopyMeshPaintTexture() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	if (StaticMeshComponents.Num() == 1 && StaticMeshComponents[0]->GetMeshPaintTexture() != nullptr && StaticMeshComponents[0]->CanMeshPaintTextureColors())
	{
		return true;
	}
	return false;
}

void UMeshPaintMode::Copy()
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	if (PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		CopyInstanceVertexColors();
	}
	else if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		CopyMeshPaintTexture();
	}
}

bool UMeshPaintMode::CanCopy() const
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	if (PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		return CanCopyInstanceVertexColors();
	}
	else if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		return CanCopyMeshPaintTexture();
	}
	return false;
}

void UMeshPaintMode::PasteMeshPaintTexture()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionPasteMeshPaintTexture", "Pasting Texture Colors"));
	FImage const& Image = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetCopiedTexture();
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->CreateComponentMeshPaintTexture(Component, Image);
	}

	bRecacheDataSizes = true;
	bRecacheValidForPaint = true;
	
	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->Refresh();
}

bool UMeshPaintMode::CanPasteMeshPaintTexture() const
{
	FImage const& Image = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetCopiedTexture();
	if (Image.GetNumPixels() == 0)
	{
		return false;
	}
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component->CanMeshPaintTextureColors())
		{
			return true;
		}
	}
	return false;
}

void UMeshPaintMode::Paste()
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	if (PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		PasteInstanceVertexColors();
	}
	else if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		PasteMeshPaintTexture();
	}
}

bool UMeshPaintMode::CanPaste() const
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	if (PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		return CanPasteInstanceVertexColors();
	}
	else if (PaletteName == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		return CanPasteMeshPaintTexture();
	}
	return false;
}

bool UMeshPaintMode::CanImportVertexColorsFromFile() const
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	return PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights;
}

void UMeshPaintMode::ImportVertexColorsFromMeshPaintTexture()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionImportVertexColorFromTextureColor", "Importing Vertex Colors From Mesh Paint Textures"));
	const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
	for (UMeshComponent* Component : MeshComponents)
	{
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->ImportVertexColorsFromMeshPaintTexture(Component);
	}

	bRecacheDataSizes = true;
}

bool UMeshPaintMode::CanImportVertexColorsFromMeshPaintTexture() const
{
	const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
	for (UMeshComponent* Component : MeshComponents)
	{
		if (Component->GetMeshPaintTexture() != nullptr)
		{
			return true;
		}
	}
	return false;
}

void UMeshPaintMode::ImportMeshPaintTextureFromVertexColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionImportTextureColorFromVertexColor", "Importing Mesh Paint Textures From Vertex Colors"));
	const TArray<UMeshComponent*> MeshComponents = GetSelectedComponents<UMeshComponent>();
	for (UMeshComponent* Component : MeshComponents)
	{
		GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->ImportMeshPaintTextureFromVertexColors(Component);
	}

	bRecacheDataSizes = true;
	bRecacheValidForPaint = true;

	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->Refresh();
}

bool UMeshPaintMode::CanImportMeshPaintTextureFromVertexColors() const
{
	const TArray<UStaticMeshComponent*> StaticMeshComponents = GetSelectedComponents<UStaticMeshComponent>();
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component->CanMeshPaintTextureColors())
		{
			return true;
		}
	}
	return false;
}

void UMeshPaintMode::FixTextureColors()
{
	FScopedTransaction Transaction(LOCTEXT("LevelMeshPainter_TransactionFixTextureColors", "Fixing Per-Instance Texture Colors"));
	GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->FixTextureColors(GetSelectedComponents<UMeshComponent>());

	bRecacheDataSizes = true;

	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->Refresh();
}

bool UMeshPaintMode::CanFixTextureColors() const
{
	return GEditor->GetEditorSubsystem<UMeshPaintModeSubsystem>()->CanFixTextureColors(GetSelectedComponents<UMeshComponent>());
}

bool UMeshPaintMode::CanCycleMeshLODs() const
{
	FName PaletteName = Toolkit->GetCurrentPalette();
	return PaletteName == UMeshPaintMode::MeshPaintMode_VertexColor || PaletteName == UMeshPaintMode::MeshPaintMode_VertexWeights;
}

bool UMeshPaintMode::GetSelectionWarning(FText& OutMessage) const
{
	FToolBuilderState SelectionState;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentSelectionState(SelectionState);

	FString ActiveTool = GetToolManager()->GetActiveToolName(EToolSide::Mouse);
	if (ActiveTool == VertexSelectToolName || ActiveTool == VertexColorPaintToolName || ActiveTool == VertexWeightPaintToolName)
	{
		for (UStaticMeshComponent* StaticMeshComponent : GetSelectedComponents<UStaticMeshComponent>())
		{
			if (!StaticMeshComponent->CanMeshPaintVertexColors())
			{
				OutMessage = LOCTEXT("LevelMeshPainter_NaniteVertexPaintWarning", "Part of the selection cannot be vertex painted because it has disabled vertex paint or because it is currently rendered with nanite.");
				return true;
			}
		}
	}
	if (ActiveTool == TextureColorSelectToolName || ActiveTool == TextureColorPaintToolName)
	{
		if (CanAddMeshPaintTextures())
		{
			OutMessage = LOCTEXT("LevelMeshPainter_AddMeshPaintTextureWarning", "Part of the selection has no Mesh Paint Texture and cannot be painted. Use the Add button to fix.");
			return true;
		}
	}

	for (AActor* SelectedActor : SelectionState.SelectedActors)
	{
		if (SelectedActor)
		{
			if (SelectedActor->IsA<ALandscapeProxy>())
			{
				TInlineComponentArray<ULandscapeNaniteComponent*> NaniteComponents;
				SelectedActor->GetComponents<ULandscapeNaniteComponent>(NaniteComponents);
				if (NaniteComponents.Num() == 0)
				{
					OutMessage = LOCTEXT("LevelMeshPainter_NaniteLandscapeWarning", "Part of the selected landscape has no nanite mesh and cannot be painted.");
					return true;
				}
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
