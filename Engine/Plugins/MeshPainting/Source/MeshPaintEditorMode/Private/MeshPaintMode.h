// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UEdMode.h"
#include "Tools/LegacyEdModeInterfaces.h"
#include "MeshPaintMode.generated.h"

class UMeshPaintingToolProperties;
class UMeshVertexPaintingToolProperties;
class UMeshVertexColorPaintingToolProperties;
class UMeshVertexWeightPaintingToolProperties;
class UMeshTexturePaintingToolProperties;
class UMeshTextureColorPaintingToolProperties;
class UMeshTextureAssetPaintingToolProperties;
class UMeshPaintModeSettings;
class IMeshPaintComponentAdapter;
class UMeshComponent;
class UMeshToolManager;

/**
 * Mesh paint Mode.  Extends editor viewports with the ability to paint data on meshes
 */
UCLASS()
class UMeshPaintMode : public UEdMode, public ILegacyEdModeViewportInterface
{
public:
	GENERATED_BODY()

	UMeshPaintMode();

	virtual void Enter() override;
	virtual void Exit() override;
	virtual void CreateToolkit() override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetModeCommands() const override;

	static UMeshPaintingToolProperties* GetToolProperties();
	static UMeshVertexPaintingToolProperties* GetVertexToolProperties();
	static UMeshVertexColorPaintingToolProperties* GetVertexColorToolProperties();
	static UMeshVertexWeightPaintingToolProperties* GetVertexWeightToolProperties();
	static UMeshTexturePaintingToolProperties* GetTextureToolProperties();
	static UMeshTextureColorPaintingToolProperties* GetTextureColorToolProperties();
	static UMeshTextureAssetPaintingToolProperties* GetTextureAssetToolProperties();
	static UMeshPaintMode* GetMeshPaintMode();
	static FName GetValidPaletteName(const FName InName);

	static FName MeshPaintMode_VertexColor;
	static FName MeshPaintMode_VertexWeights;
	static FName MeshPaintMode_TextureColor;
	static FName MeshPaintMode_TextureAsset;
	static FString VertexSelectToolName;
	static FString TextureColorSelectToolName;
	static FString TextureAssetSelectToolName;
	static FString VertexColorPaintToolName;
	static FString VertexWeightPaintToolName;
	static FString TextureColorPaintToolName;
	static FString TextureAssetPaintToolName;

	/** Returns the instance of ComponentClass found in the current Editor selection */
	template<typename ComponentClass>
	TArray<ComponentClass*> GetSelectedComponents() const;

	/** Returns data size of per-instance vertex color data for the currently selected components. */
	uint32 GetVertexDataSizeInBytes() const { return CachedVertexDataSize; }
	/** Returns resource size of mesh paint textures for the currently selected components. */
	uint32 GetMeshPaintTextureResourceSizeInBytes() const { return CachedMeshPaintTextureResourceSize; }

	/** Returns true and fills the message if there is the current selection is not in an expectd state for the current painting mode. */
	bool GetSelectionWarning(FText& OutMessage) const;

protected:
	/** Binds UI commands to actions for the mesh paint mode */
	virtual void BindCommands() override;

	// UEdMode interface
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void ActorSelectionChangeNotify() override;
	virtual void ElementSelectionChangeNotify() override;
	virtual void ActorPropChangeNotify() override;
	virtual void ActivateDefaultTool() override;
	virtual void UpdateOnPaletteChange(FName NewPalette);
	// end UEdMode Interface
	

	void UpdateSelectedMeshes();
	void UpdateOnMaterialChange(bool bInvalidateHitProxies);
	void UpdateOnPostCompile(const TArray<struct FAssetCompileData>& Data);
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstanceMap);
	void OnResetViewMode();
	void OnVertexPaintFinished();
	void OnTextureColorVertexPaintFinished(UMeshComponent* MeshComponent);

	void UpdateCachedDataSizes();
	void EndPaintToolIfNoLongerValid();

	bool IsInSelectTool() const;
	bool IsInPaintTool() const;

	// Start command bindings
	void SwapColors();
	bool CanSwapColors() const;
	void FillVertexColors();
	bool CanFillVertexColors() const;
	void FillTexture();
	bool CanFillTexture() const;
	void PropagateVertexColorsToMesh();
	bool CanPropagateVertexColorsToMesh() const;
	void PropagateVertexColorsToLODs();
	bool CanPropagateVertexColorsToLODs() const;
	void SaveVertexColorsToAssets();
	bool CanSaveVertexColorsToAssets() const;
	void SaveTexturePackages();
	bool CanSaveTexturePackages() const;
	void AddMeshPaintTextures();
	bool CanAddMeshPaintTextures() const;
	void RemoveInstanceVertexColors();
	bool CanRemoveInstanceVertexColors() const;
	void RemoveMeshPaintTexture();
	bool CanRemoveMeshPaintTextures() const;
	void CopyInstanceVertexColors();
	bool CanCopyInstanceVertexColors() const;
	void CopyMeshPaintTexture();
	bool CanCopyMeshPaintTexture() const;
	void Copy();
	bool CanCopy() const;
	void PasteInstanceVertexColors();
	bool CanPasteInstanceVertexColors() const;
	void PasteMeshPaintTexture();
	bool CanPasteMeshPaintTexture() const;
	void Paste();
	bool CanPaste() const;
	void ImportVertexColorsFromFile();
	bool CanImportVertexColorsFromFile() const;
	void ImportVertexColorsFromMeshPaintTexture();
	bool CanImportVertexColorsFromMeshPaintTexture() const;
	void ImportMeshPaintTextureFromVertexColors();
	bool CanImportMeshPaintTextureFromVertexColors() const;
	void FixVertexColors();
	bool CanFixVertexColors() const;
	void FixTextureColors();
	bool CanFixTextureColors() const;
	void CycleMeshLODs(int32 Direction);
	bool CanCycleMeshLODs() const;
	void CycleTextures(int32 Direction);
	bool CanCycleTextures() const;
	void ChangeBrushRadius(int32 Direction);
	void ChangeBrushStrength(int32 Direction);
	void ChangeBrushFalloff(int32 Direction);
	bool CanChangeBrush() const;
	// End command bindings

protected:
	UPROPERTY(Transient)
	TObjectPtr<UMeshPaintModeSettings> ModeSettings;

	bool bRecacheDataSizes = false;
	uint32 CachedVertexDataSize = 0;
	uint32 CachedMeshPaintTextureResourceSize = 0;
	
	bool bRecacheValidForPaint = false;

	FDelegateHandle PaletteChangedHandle;
	FConsoleVariableSinkHandle CVarDelegateHandle;
};
