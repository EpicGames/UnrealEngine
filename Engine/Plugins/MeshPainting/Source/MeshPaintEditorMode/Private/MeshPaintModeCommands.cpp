// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintModeCommands.h"

#include "MeshPaintMode.h"
#include "MeshTexturePaintingTool.h"
#include "MeshVertexPaintingTool.h"

#define LOCTEXT_NAMESPACE "MeshPaintEditorModeCommands"

void FMeshPaintingToolActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	ToolCDOs.Add(GetMutableDefault<USingleSelectionTool>());
	ToolCDOs.Add(GetMutableDefault<UMeshVertexColorPaintingTool>());
}

void FMeshPaintingToolActionCommands::RegisterAllToolActions()
{
	FMeshPaintingToolActionCommands::Register();
}

void FMeshPaintingToolActionCommands::UnregisterAllToolActions()
{
	FMeshPaintingToolActionCommands::Unregister();
}

void FMeshPaintingToolActionCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
	if (FMeshPaintingToolActionCommands::IsRegistered())
	{
		!bUnbind ? FMeshPaintingToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool) : FMeshPaintingToolActionCommands::Get().UnbindActiveCommands(UICommandList);
	}
}

void FMeshPaintEditorModeCommands::RegisterCommands()
{
	UI_COMMAND(SelectVertex, "Select", "Select the mesh for vertex painting", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SelectTextureColor, "Select", "Select the mesh for texture color painting", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SelectTextureAsset, "Select", "Select the mesh for texture asset painting", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(PaintVertexColor, "Paint", "Paint the mesh vertex colors", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(PaintVertexWeight, "Paint", "Paint the mesh vertex weights", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(PaintTextureColor, "Paint", "Paint the mesh texture colors", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(PaintTextureAsset, "Paint", "Paint texture assets used by the mesh material", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SwapColor, "Swap", "Switches the foreground and background colors used for painting", EUserInterfaceActionType::Button, FInputChord(EKeys::X));
	UI_COMMAND(FillVertex, "Fill", "Fills the selected meshes with the paint color", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FillTexture, "Fill", "Fills the selected textures with the paint color", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PropagateMesh, "ToMesh", "Applies per instance vertex colors to the source meshes", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PropagateLODs, "ToLODs", "Applies the vertex colors from LOD0 to all LOD levels", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveVertex, "Save", "Saves the source meshes for the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveTexture, "Save", "Saves the modified textures for the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Add, "Add", "Adds mesh paint textures to the selected mesh components to enable painting", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveVertex, "Remove", "Removes any vertex colors from the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveTexture, "Remove", "Removes any mesh paint textures from the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Copy, "Copy", "Copies colors from the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Paste, "Paste", "Pastes colors on the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Import, "Import", "Imports vertex colors from a TGA texture file to the selected meshes", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GetVertexColors, "Vertex", "Imports texture colors from vertex colors on the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GetTextureColors, "Texture", "Imports vertex colors from texture colors on the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FixVertex, "Fix", "Applies any required color data fixes to the selected mesh components", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FixTexture, "Fix", "Applies any pending resolution change of texture color painting", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PreviousLOD, "Previous LOD", "Cycles to the previous possible mesh LOD to paint on", EUserInterfaceActionType::Button, FInputChord(EKeys::B));
	UI_COMMAND(NextLOD, "Next LOD", "Cycles to the next possible mesh LOD to paint on", EUserInterfaceActionType::Button, FInputChord(EKeys::N));
	UI_COMMAND(PreviousTexture, "Previous Texture", "Cycle To previous texture", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma));
	UI_COMMAND(NextTexture, "Next Texture", "Cycle To next texture", EUserInterfaceActionType::Button, FInputChord(EKeys::Period));
	UI_COMMAND(IncreaseBrushRadius, "Increase Brush Radius", "Increase brush radius by a percentage of its current size.", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket));
	UI_COMMAND(DecreaseBrushRadius, "Decrease Brush Size", "Decrease brush radius by a percentage of its current size.", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket));
	UI_COMMAND(IncreaseBrushStrength, "Increase Brush Strength", "Increase brush strength by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket));
	UI_COMMAND(DecreaseBrushStrength, "Decrease Brush Strength", "Decrease brush strength by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket));
	UI_COMMAND(IncreaseBrushFalloff, "Increase Brush Falloff", "Increase brush falloff by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket)));
	UI_COMMAND(DecreaseBrushFalloff, "Decrease Brush Falloff", "Decrease brush falloff by a fixed increment.", EUserInterfaceActionType::Button, FInputChord(FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket)));

	TArray<TSharedPtr<FUICommandInfo>> VertexColorCommands = {
		SelectVertex, PaintVertexColor, SwapColor, FillVertex, PropagateLODs, PropagateMesh, SaveVertex, RemoveVertex, Copy, Paste, Import, GetTextureColors, FixVertex
	};
	Commands.Add(UMeshPaintMode::MeshPaintMode_VertexColor, VertexColorCommands);

	TArray<TSharedPtr<FUICommandInfo>> VertexWeightCommands = {
		SelectVertex, PaintVertexWeight, FillVertex, PropagateLODs, PropagateMesh, SaveVertex, RemoveVertex, Copy, Paste, Import, FixVertex
	};
	Commands.Add(UMeshPaintMode::MeshPaintMode_VertexWeights, VertexWeightCommands);

	TArray<TSharedPtr<FUICommandInfo>> TextureColorCommands = {
		SelectTextureColor, PaintTextureColor, SwapColor, FillTexture, SaveTexture, Add, RemoveTexture, Copy, Paste, GetVertexColors, FixTexture
	};
	Commands.Add(UMeshPaintMode::MeshPaintMode_TextureColor, TextureColorCommands);

	TArray<TSharedPtr<FUICommandInfo>> TextureAssetCommands = {
		SelectTextureAsset, PaintTextureAsset, SwapColor, FillTexture, SaveTexture
	};
	Commands.Add(UMeshPaintMode::MeshPaintMode_TextureAsset, TextureAssetCommands);
}

#undef LOCTEXT_NAMESPACE
