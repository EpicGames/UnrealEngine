// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintModeHelpers.h"

#include "ComponentReregisterContext.h"


#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "IDesktopPlatform.h"
#include "SceneView.h"
#include "InteractiveToolManager.h"
#include "StaticMeshComponentLODInfo.h"

#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include "Rendering/SkeletalMeshLODModel.h"
#include "Utils.h"

#include "Framework/Application/SlateApplication.h"
#include "ImportVertexColorOptions.h"
#include "EditorViewportClient.h"
#include "Interfaces/IMainFrameModule.h"

#include "DesktopPlatformModule.h"
#include "PackageTools.h"

#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "EditorViewportClient.h"

#include "Factories/FbxSkeletalMeshImportData.h"

#include "Rendering/SkeletalMeshModel.h"
#include "MeshPaintHelpers.h"
#include "MeshPaintMode.h"
#include "MeshPaintVisualize.h"
#include "MeshTexturePaintingTool.h"
#include "MeshVertexPaintingTool.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangePythonPipelineBase.h"

#include "Math/Color.h"
#include "StaticMeshLODResourcesToDynamicMesh.h"
#include "Sampling/MeshMapBaker.h"
#include "Sampling/MeshPropertyMapEvaluator.h"
#include "VT/MeshPaintVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintModeHelpers)


void UMeshPaintModeSubsystem::SetViewportColorMode(EMeshPaintActiveMode ActiveMode, EMeshPaintDataColorViewMode ColorViewMode, FEditorViewportClient* ViewportClient, UInteractiveTool const* ActiveTool)
{
	if (ViewportClient->IsPerspective())
	{
		// Update viewport show flags
		{
			// show flags forced on during vertex color modes
			if (ColorViewMode == EMeshPaintDataColorViewMode::Normal)
			{
				if (ViewportClient->EngineShowFlags.VertexColors)
				{
					// Clear the flags relevant to vertex color modes
					ViewportClient->EngineShowFlags.SetVertexColors(false);
					ViewportClient->CurrentNaniteVisualizationMode = NAME_None;

					// Restore the vertex color mode flags that were set when we last entered vertex color mode
					ApplyViewMode(ViewportClient->GetViewMode(), ViewportClient->IsPerspective(), ViewportClient->EngineShowFlags);
					
					MeshPaintVisualize::SetPaintMode(EMeshPaintVisualizePaintMode::VertexColor);
					MeshPaintVisualize::SetShowMode(EMeshPaintVisualizeShowMode::ShowAll);
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Color);
					MeshPaintVisualize::SetTextureAsset(nullptr);
				}
			}
			else
			{
				ViewportClient->EngineShowFlags.SetVertexColors(true);

				MeshPaintVisualize::SetShowMode(EMeshPaintVisualizeShowMode::ShowSelected);

				switch (ColorViewMode)
				{
				case EMeshPaintDataColorViewMode::RGB:
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Color);
					break;
				case EMeshPaintDataColorViewMode::Alpha:
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Alpha);
					break;
				case EMeshPaintDataColorViewMode::Red:
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Red);
					break;
				case EMeshPaintDataColorViewMode::Green:
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Green);
					break;
				case EMeshPaintDataColorViewMode::Blue:
					MeshPaintVisualize::SetChannelMode(EVertexColorViewMode::Blue);
					break;
				}

				UTexture* SelectedTexture = nullptr;
				int32 UVChannel = 0;

				if (ActiveMode == EMeshPaintActiveMode::Texture)
				{
					UMeshTextureAssetPaintingTool const* TextureTool = Cast<UMeshTextureAssetPaintingTool>(ActiveTool);
					if (TextureTool != nullptr)
					{
						SelectedTexture = TextureTool->GetSelectedPaintTextureWithOverride();
						UVChannel = TextureTool->GetSelectedUVChannel(nullptr);
					}
				}

				static FName NAME_VertexColor("VertexColor");
				static FName NAME_MeshPaintTexture("MeshPaintTexture");

				switch (ActiveMode)
				{
				case EMeshPaintActiveMode::VertexColor:
				case EMeshPaintActiveMode::VertexWeights:
					MeshPaintVisualize::SetPaintMode(EMeshPaintVisualizePaintMode::VertexColor);
					ViewportClient->EngineShowFlags.SetVisualizeNanite(true);
					ViewportClient->CurrentNaniteVisualizationMode = NAME_VertexColor;
					break;
				case EMeshPaintActiveMode::TextureColor:
					MeshPaintVisualize::SetPaintMode(EMeshPaintVisualizePaintMode::TextureColor);
					ViewportClient->EngineShowFlags.SetVisualizeNanite(true);
					ViewportClient->CurrentNaniteVisualizationMode = NAME_MeshPaintTexture;
					break;
				case EMeshPaintActiveMode::Texture:
					MeshPaintVisualize::SetPaintMode(EMeshPaintVisualizePaintMode::TextureAsset);
					ViewportClient->EngineShowFlags.SetVisualizeNanite(SelectedTexture != nullptr);
					ViewportClient->CurrentNaniteVisualizationMode = SelectedTexture != nullptr ? NAME_MeshPaintTexture : NAME_None;
					break;
				}

				MeshPaintVisualize::SetTextureAsset(SelectedTexture);
				MeshPaintVisualize::SetTextureCoordinateIndex(UVChannel);
			}
		}
	}
}

void UMeshPaintModeSubsystem::SetRealtimeViewport(FEditorViewportClient* ViewportClient, bool bRealtime)
{
	if (ViewportClient != nullptr)
	{
		if (ViewportClient->IsPerspective())
		{
			static const FText SystemDisplayName = NSLOCTEXT("MeshPaint", "RealtimeOverrideMessage_MeshPaint", "Mesh Paint");
			if (bRealtime)
			{
				if (!ViewportClient->HasRealtimeOverride(SystemDisplayName))
				{
					ViewportClient->AddRealtimeOverride(bRealtime, SystemDisplayName);
				}
			}
			else
			{
				if (ViewportClient->HasRealtimeOverride(SystemDisplayName))
				{
					ViewportClient->RemoveRealtimeOverride(SystemDisplayName);
				}
			}
		}
	}
}


void UMeshPaintModeSubsystem::ImportVertexColorsFromTexture(UMeshComponent* MeshComponent)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid mesh component ptr"));

	// Get TGA texture filepath
	FString ChosenFilename("");
	FString ExtensionStr;
	ExtensionStr += TEXT("TGA Files|*.tga|");

	FString PromptTitle("Pick TGA Texture File");

	// First, display the file open dialog for selecting the file.
	TArray<FString> Filenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			PromptTitle,
			TEXT(""),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (bOpen && Filenames.Num() == 1)
	{
		// Valid file name picked
		const FString FileName = Filenames[0];
		UTexture2D* ColorTexture = ImportObject<UTexture2D>(GEngine, NAME_None, RF_Public, *FileName, nullptr, nullptr, TEXT("NOMIPMAPS=1 NOCOMPRESSION=1"));

		if (ColorTexture && ColorTexture->Source.GetFormat() == TSF_BGRA8)
		{
			// Have a valid texture, now need user to specify options for importing
			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(FText::FromString(TEXT("Vertex Color Import Options")))
				.SizingRule(ESizingRule::Autosized);

			TSharedPtr<SImportVertexColorOptionsWindow> OptionsWindow = SNew(SImportVertexColorOptionsWindow).WidgetWindow(Window)
				.WidgetWindow(Window)
				.Component(MeshComponent)
				.FullPath(FText::FromString(ChosenFilename));

			Window->SetContent
			(
				OptionsWindow->AsShared()
			);

			TSharedPtr<SWindow> ParentWindow;
			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}
			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			if (OptionsWindow->ShouldImport())
			{
				// Options specified and start importing
				UImportVertexColorOptions* Options = OptionsWindow->GetOptions();

				if (MeshComponent->IsA<UStaticMeshComponent>())
				{
					UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);
					if (StaticMeshComponent)
					{
						if (Options->bImportToInstance)
						{
							// Import colors to static mesh / component
							ImportVertexColorsToStaticMeshComponent(StaticMeshComponent, Options, ColorTexture);
						}
						else
						{
							if (StaticMeshComponent->GetStaticMesh())
							{
								ImportVertexColorsToStaticMesh(StaticMeshComponent->GetStaticMesh(), Options, ColorTexture);
							}
						}
					}
				}
				else if (MeshComponent->IsA<USkeletalMeshComponent>())
				{
					USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent);

					if (SkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						// Import colors to skeletal mesh
						ImportVertexColorsToSkeletalMesh(SkeletalMeshComponent->GetSkeletalMeshAsset(), Options, ColorTexture);
					}
				}
			}
		}
		else if (!ColorTexture)
		{
			// Unable to import file
		}
		else if (ColorTexture && ColorTexture->Source.GetFormat() != TSF_BGRA8)
		{
			// Able to import file but incorrect format
		}
	}
}

void UMeshPaintModeSubsystem::ImportVertexColorsFromMeshPaintTexture(UMeshComponent* MeshComponent)
{
	if (UTexture2D* Texture = Cast<UTexture2D>(MeshComponent->GetMeshPaintTexture()))
	{
#if WITH_EDITOR
		// We may need to wait for the texture to compile before importing.
		// This is most likely to happen when we are immediately propagating texture color painting to vertex colors.
		Texture->BlockOnAnyAsyncBuild();
#endif

		UImportVertexColorOptions* Options = NewObject<UImportVertexColorOptions>();
 		Options->UVIndex = MeshComponent->GetMeshPaintTextureCoordinateIndex();

		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			ImportVertexColorsToStaticMeshComponent(StaticMeshComponent, Options, Texture);
		}
	}
}

void UMeshPaintModeSubsystem::ImportMeshPaintTextureFromVertexColors(UMeshComponent* MeshComponent)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);
	if (StaticMeshComponent == nullptr)
	{
		return;
	}

	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return;
	}

	const int32 LodIndex = 0;

	FStaticMeshComponentLODInfo* InstanceMeshLODInfo = StaticMeshComponent->LODData.IsValidIndex(LodIndex) ? &StaticMeshComponent->LODData[LodIndex] : nullptr;
	const bool bHasPerInstanceVertexColors = InstanceMeshLODInfo != nullptr && InstanceMeshLODInfo->OverrideVertexColors != nullptr;

	UE::Geometry::FStaticMeshLODResourcesToDynamicMesh::ConversionOptions ConversionOptions;
	ConversionOptions.bWantTangents = false;
	ConversionOptions.bWantMaterialIDs = false;

	UE::Geometry::FDynamicMesh3 DynamicMesh;
	UE::Geometry::FStaticMeshLODResourcesToDynamicMesh Converter;
	Converter.Convert(
		&StaticMesh->GetRenderData()->LODResources[LodIndex],
		ConversionOptions,
		DynamicMesh,
		bHasPerInstanceVertexColors,
		[InstanceMeshLODInfo](int32 Index)
		{
			return InstanceMeshLODInfo->OverrideVertexColors->VertexColor(Index);
		});

	const int32 TextureSize = StaticMeshComponent->GetMeshPaintTextureResolution();

	const UE::Geometry::FDynamicMeshAABBTree3 DetailSpatial(&DynamicMesh);
	UE::Geometry::FMeshBakerDynamicMeshSampler DetailSampler(&DynamicMesh, &DetailSpatial);

	TSharedPtr<UE::Geometry::FMeshPropertyMapEvaluator, ESPMode::ThreadSafe> PropertyEval = MakeShared<UE::Geometry::FMeshPropertyMapEvaluator, ESPMode::ThreadSafe>();
	PropertyEval->Property = UE::Geometry::EMeshPropertyMapType::VertexColor;

	UE::Geometry::FMeshMapBaker Baker;
	Baker.SetTargetMesh(&DynamicMesh);
	Baker.SetDetailSampler(&DetailSampler);
	Baker.AddEvaluator(PropertyEval);
	Baker.SetTargetMeshUVLayer(StaticMeshComponent->GetMeshPaintTextureCoordinateIndex());
	Baker.SetDimensions(UE::Geometry::FImageDimensions(TextureSize, TextureSize));
	Baker.SetProjectionDistance(3.0f);
	Baker.SetSamplesPerPixel(1);
	Baker.SetFilter(UE::Geometry::FMeshMapBaker::EBakeFilterType::BSpline);
	Baker.SetGutterEnabled(true);
	Baker.SetGutterSize(4);
	Baker.Bake();

	FImageView ResultImage((FLinearColor*)Baker.GetBakeResults(0)[0]->GetImageBuffer().GetData(), TextureSize, TextureSize);
	FImage ConvertedImage;
	ResultImage.CopyTo(ConvertedImage, ERawImageFormat::BGRA8, EGammaSpace::sRGB);

	UMeshPaintVirtualTexture* NewTexture = NewObject<UMeshPaintVirtualTexture>(StaticMeshComponent);
	NewTexture->Source.Init(ConvertedImage);
	NewTexture->OwningComponent = MakeWeakObjectPtr(StaticMeshComponent);
	NewTexture->UpdateResource();

	StaticMeshComponent->Modify();
	StaticMeshComponent->SetMeshPaintTexture(NewTexture);
}


void UMeshPaintModeSubsystem::ImportVertexColorsToSkeletalMesh(USkeletalMesh* SkeletalMesh, const UImportVertexColorOptions* Options, UTexture2D* Texture)
{
	checkf(SkeletalMesh && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: better to use GetMipImage rather than GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext;
	FSkeletalMeshRenderData* Resource = SkeletalMesh->GetResourceForRendering();
	const int32 ImportLOD = Options->LODIndex;
	const int32 UVIndex = Options->UVIndex;
	const FColor ColorMask = Options->CreateColorMask();
	if (Resource && Resource->LODRenderData.IsValidIndex(ImportLOD))
	{
		RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(SkeletalMesh);
		SkeletalMesh->Modify();
		SkeletalMesh->ReleaseResources();
		SkeletalMesh->ReleaseResourcesFence.Wait();

		FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[ImportLOD];

		if (LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
		{
			LODData.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, LODData.GetNumVertices());
			BeginInitResource(&LODData.StaticVertexBuffers.ColorVertexBuffer);
		}

		for (uint32 VertexIndex = 0; VertexIndex < LODData.GetNumVertices(); ++VertexIndex)
		{
			const FVector2D UV = FVector2D(LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
			LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
		}

		SkeletalMesh->InitResources();
	}


	checkf(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(ImportLOD), TEXT("Invalid Imported Model index for vertex painting"));
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[ImportLOD];
	const uint32 NumVertices = LODModel.NumVertices;
	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODModel.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

		const FVector2D UV = FVector2D(LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].UVs[UVIndex]);
		LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
	}

	//Make sure we change the import data so the re-import do not replace the new data
	if (SkeletalMesh->GetAssetImportData())
	{
		UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
		if (ImportData && ImportData->VertexColorImportOption != EVertexColorImportOption::Ignore)
		{
			ImportData->SetFlags(RF_Transactional);
			ImportData->Modify();
			ImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}

		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData());
		if (InterchangeAssetImportData)
		{
			TArray<UObject*> Pipelines = InterchangeAssetImportData->GetPipelines();
			for (UObject* PipelineBase : Pipelines)
			{
				UInterchangeGenericAssetsPipeline* GenericAssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(PipelineBase);

				if (GenericAssetPipeline)
				{
					if (GenericAssetPipeline->CommonMeshesProperties && GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption != EInterchangeVertexColorImportOption::IVCIO_Ignore)
					{
						GenericAssetPipeline->SetFlags(RF_Transactional);
						GenericAssetPipeline->Modify();
						GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
					}
				}
			}
		}
	}
}


bool UMeshPaintModeSubsystem::RetrieveViewportPaintRays(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, TArray<FPaintRay>& OutPaintRays)
{
	checkf(View && Viewport && PDI, TEXT("Invalid Viewport data"));
	FEditorViewportClient* ViewportClient = (FEditorViewportClient*)Viewport->GetClient();
	checkf(ViewportClient != nullptr, TEXT("Unable to retrieve viewport client"));

	if (ViewportClient->IsPerspective())
	{
		{
			// Else we're painting with mouse
			// Make sure the cursor is visible OR we're flood filling.  No point drawing a paint cue when there's no cursor.
			if (Viewport->IsCursorVisible())
			{
				if (!PDI->IsHitTesting())
				{
					// Grab the mouse cursor position
					FIntPoint MousePosition;
					Viewport->GetMousePos(MousePosition);

					// Is the mouse currently over the viewport? or flood filling
					if ((MousePosition.X >= 0 && MousePosition.Y >= 0 && MousePosition.X < (int32)Viewport->GetSizeXY().X && MousePosition.Y < (int32)Viewport->GetSizeXY().Y))
					{
						// Compute a world space ray from the screen space mouse coordinates
						FViewportCursorLocation MouseViewportRay(View, ViewportClient, MousePosition.X, MousePosition.Y);

						FPaintRay& NewPaintRay = *new(OutPaintRays) FPaintRay();
						NewPaintRay.CameraLocation = View->ViewMatrices.GetViewOrigin();
						NewPaintRay.RayStart = MouseViewportRay.GetOrigin();
						NewPaintRay.RayDirection = MouseViewportRay.GetDirection();
						NewPaintRay.ViewportInteractor = nullptr;
					}
				}
			}
		}
	}

	return false;
}


void UMeshPaintModeSubsystem::ImportVertexColorsToStaticMesh(UStaticMesh* StaticMesh, const UImportVertexColorOptions* Options, UTexture2D* Texture)
{
	checkf(StaticMesh && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: better to use GetMipImage rather than GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FStaticMeshComponentRecreateRenderStateContext > RecreateRenderStateContext = MakeUnique<FStaticMeshComponentRecreateRenderStateContext>(StaticMesh);
	const int32 ImportLOD = Options->LODIndex;
	FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[ImportLOD];

	// Dirty the mesh
	StaticMesh->Modify();

	// Release the static mesh's resources.
	StaticMesh->ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	StaticMesh->ReleaseResourcesFence.Wait();

	if (LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
	{
		// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
		LODModel.VertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, LODModel.GetNumVertices());

		// @todo MeshPaint: Make sure this is the best place to do this
		BeginInitResource(&LODModel.VertexBuffers.ColorVertexBuffer);
	}

	const int32 UVIndex = Options->UVIndex;
	const FColor ColorMask = Options->CreateColorMask();
	for (uint32 VertexIndex = 0; VertexIndex < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices(); ++VertexIndex)
	{
		const FVector2D UV = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
		LODModel.VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
	}

	// Make sure colors are saved into raw mesh

	StaticMesh->InitResources();
}

void UMeshPaintModeSubsystem::ImportVertexColorsToStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, const UImportVertexColorOptions* Options, UTexture2D* Texture)
{
	checkf(StaticMeshComponent && Options && Texture, TEXT("Invalid ptr"));

	// Extract color data from texture
	// todo: better to use GetMipImage rather than GetMipData
	TArray64<uint8> SrcMipData;
	verify( Texture->Source.GetMipData(SrcMipData, 0) );
	const uint8* MipData = SrcMipData.GetData();

	TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
	const UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();
	if (Mesh)
	{
		ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(StaticMeshComponent);
		StaticMeshComponent->Modify();

		const int32 ImportLOD = Options->LODIndex;
		const FStaticMeshLODResources& LODModel = Mesh->GetRenderData()->LODResources[ImportLOD];

		if (!StaticMeshComponent->LODData.IsValidIndex(ImportLOD))
		{
			StaticMeshComponent->SetLODDataCount(ImportLOD + 1, StaticMeshComponent->LODData.Num());
		}

		FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[ImportLOD];

		if (InstanceMeshLODInfo.OverrideVertexColors)
		{
			InstanceMeshLODInfo.ReleaseOverrideVertexColorsAndBlock();
		}

		// Setup the instance vertex color array
		InstanceMeshLODInfo.OverrideVertexColors = new FColorVertexBuffer;

		if ((int32)LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices() == LODModel.GetNumVertices())
		{
			// copy mesh vertex colors to the instance ones
			InstanceMeshLODInfo.OverrideVertexColors->InitFromColorArray(&LODModel.VertexBuffers.ColorVertexBuffer.VertexColor(0), LODModel.GetNumVertices());
		}
		else
		{
			// Original mesh didn't have any colors, so just use a default color
			InstanceMeshLODInfo.OverrideVertexColors->InitFromSingleColor(FColor::White, LODModel.GetNumVertices());
		}

		if (ImportLOD > 0)
		{
			StaticMeshComponent->bCustomOverrideVertexColorPerLOD = true;
		}

		const int32 UVIndex = Options->UVIndex;
		const FColor ColorMask = Options->CreateColorMask();
		for (uint32 VertexIndex = 0; VertexIndex < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices(); ++VertexIndex)
		{
			const FVector2D UV = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex));
			InstanceMeshLODInfo.OverrideVertexColors->VertexColor(VertexIndex) = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PickVertexColorFromTextureData(MipData, UV, Texture, ColorMask);
		}

		//Update the cache painted vertices
		InstanceMeshLODInfo.PaintedVertices.Empty();
		StaticMeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(InstanceMeshLODInfo.OverrideVertexColors);
	}
	else
	{
		// Error
	}
}

void UMeshPaintModeSubsystem::PropagateVertexColors(const TArray<UStaticMeshComponent *> StaticMeshComponents)
{
	bool SomePaintWasPropagated = false;
	TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		UStaticMesh* Mesh = Component->GetStaticMesh();
		for (int32 LODIndex = 0; LODIndex < Mesh->GetRenderData()->LODResources.Num(); LODIndex++)
		{
			// Will not be guaranteed to match render data as user can paint to a specific LOD index
			if (Component->LODData.IsValidIndex(LODIndex))
			{
				FStaticMeshComponentLODInfo& InstanceMeshLODInfo = Component->LODData[LODIndex];
				if (InstanceMeshLODInfo.OverrideVertexColors)
				{
					Mesh->Modify();
					// Try using the mapping generated when building the mesh.
					if (GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PropagateColorsToRawMesh(Mesh, LODIndex, InstanceMeshLODInfo))
					{
						SomePaintWasPropagated = true;
					}
				}
			}
		}

		if (SomePaintWasPropagated)
		{
			ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component);
			GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->RemoveComponentInstanceVertexColors(Component);
			Mesh->Build();
		}
	}
}

bool UMeshPaintModeSubsystem::CanPropagateVertexColors(TArray<UStaticMeshComponent*>& StaticMeshComponents, TArray<UStaticMesh*>& StaticMeshes, int32 NumInstanceVertexColorBytes)
{
	bool bValid = StaticMeshComponents.Num() > 0;
	for (const UStaticMeshComponent* Component : StaticMeshComponents)
	{
		UStaticMesh* StaticMesh = Component->GetStaticMesh();
		// Check for components painting to the same static mesh
		const bool bDuplicateSelection = StaticMesh != nullptr && StaticMeshes.Contains(StaticMesh);

		if (bDuplicateSelection)
		{
			bValid = false;
			break;
		}

		if (StaticMesh != nullptr)
		{
			// Disallow propagation of vertex colors to cooked static mesh assets.
			if (StaticMesh->GetOutermost()->bIsCookedForEditor)
			{
				bValid = false;
				break;
			}
			
			StaticMeshes.AddUnique(StaticMesh);
		}

		int32 CachedLODIndex = 0;
		if (UMeshVertexColorPaintingTool* ColorPaintingTool = Cast<UMeshVertexColorPaintingTool>(UMeshPaintMode::GetMeshPaintMode()->GetToolManager()->GetActiveTool(EToolSide::Left)))
		{
			CachedLODIndex = ColorPaintingTool->GetCachedLODIndex();
		}

		GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetInstanceColorDataInfo(Component, CachedLODIndex, NumInstanceVertexColorBytes);
	}

	return bValid && (NumInstanceVertexColorBytes > 0);
}

void UMeshPaintModeSubsystem::CopyVertexColors(const TArray<UStaticMeshComponent *> StaticMeshComponents, TArray<FPerComponentVertexColorData>& CopiedVertexColors)
{
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		/** Make sure we have valid data to copy from */
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		const UStaticMesh* StaticMesh = Component->GetStaticMesh();
		ensure(StaticMesh != nullptr);
		if (StaticMesh)
		{
			// Create copy structure instance for this mesh 
			FPerComponentVertexColorData ComponentData(StaticMesh, Component->GetBlueprintCreatedComponentIndex());
			const int32 NumLODs = StaticMesh->GetNumLODs();
			ComponentData.PerLODVertexColorData.AddDefaulted(NumLODs);

			// Retrieve and store vertex colors for each LOD in the mesh 
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				FPerLODVertexColorData& LODData = ComponentData.PerLODVertexColorData[LODIndex];

				TArray<FColor> ColorData;
				TArray<FVector> VertexData;

				if (Component->LODData.IsValidIndex(LODIndex) && (Component->LODData[LODIndex].OverrideVertexColors != nullptr))
				{
					ColorData = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetInstanceColorDataForLOD(Component, LODIndex);
				}
				else
				{
					ColorData = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetColorDataForLOD(StaticMesh, LODIndex);
				}
				VertexData = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetVerticesForLOD(StaticMesh, LODIndex);

				const bool bValidColorData = VertexData.Num() == ColorData.Num();
				for (int32 VertexIndex = 0; VertexIndex < VertexData.Num(); ++VertexIndex)
				{
					const FColor& Color = bValidColorData ? ColorData[VertexIndex] : FColor::White;
					LODData.ColorsByIndex.Add(Color);
					LODData.ColorsByPosition.Add(VertexData[VertexIndex], Color);
				}
			}

			CopiedVertexColors.Add(ComponentData);
		}
	}
}

bool UMeshPaintModeSubsystem::CanCopyInstanceVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, int32 PaintingMeshLODIndex)
{
	// Ensure that the selection does not contain two components which point to identical meshes
	TArray<const UStaticMesh*> ContainedMeshes;

	bool bValidSelection = true;
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		if (Component->GetStaticMesh() != nullptr)
		{
			const UStaticMesh* StaticMesh = Component->GetStaticMesh();
			if (!ContainedMeshes.Contains(StaticMesh))
			{
				ContainedMeshes.Add(StaticMesh);
			}
			else
			{
				bValidSelection = false;
				break;
			}
		}
	}

	int32 NumValidMeshes = 0;
	// Retrieve per instance vertex color information (only valid if the component contains actual instance vertex colors)
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		if (Component->GetStaticMesh() != nullptr && Component->GetStaticMesh()->GetNumLODs() > (int32)PaintingMeshLODIndex)
		{
			uint32 BufferSize = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetVertexColorBufferSize(Component, PaintingMeshLODIndex, true);

			if (BufferSize > 0)
			{
				++NumValidMeshes;
			}
		}
	}

	return bValidSelection && (NumValidMeshes != 0);
}

void UMeshPaintModeSubsystem::PasteVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, TArray<FPerComponentVertexColorData>& CopiedColorsByComponent)
{
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		UStaticMesh* Mesh = Component->GetStaticMesh();
		if (Mesh && Mesh->GetNumLODs() > 0)
		{
			// See if there is a valid instance of copied vertex colors for this mesh
			const int32 BlueprintCreatedComponentIndex = Component->GetBlueprintCreatedComponentIndex();
			FPerComponentVertexColorData* PasteColors = CopiedColorsByComponent.FindByPredicate([=](const FPerComponentVertexColorData& ComponentData)
			{
				return (ComponentData.OriginalMesh.Get() == Mesh && ComponentData.ComponentIndex == BlueprintCreatedComponentIndex);
			});

			if (PasteColors)
			{
				ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component);

				const int32 NumLods = Mesh->GetNumLODs();
				Component->SetFlags(RF_Transactional);
				Component->Modify();
				Component->SetLODDataCount(NumLods, NumLods);
				/** Remove all vertex colors before we paste in new ones */
				GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->RemoveComponentInstanceVertexColors(Component);

				/** Try and apply copied vertex colors for each LOD in the mesh */
				for (int32 LODIndex = 0; LODIndex < NumLods; ++LODIndex)
				{
					FStaticMeshLODResources& LodRenderData = Mesh->GetRenderData()->LODResources[LODIndex];
					FStaticMeshComponentLODInfo& ComponentLodInfo = Component->LODData[LODIndex];

					const int32 NumLodsInCopyBuffer = PasteColors->PerLODVertexColorData.Num();
					if (LODIndex >= NumLodsInCopyBuffer)
					{
						// no corresponding LOD in color paste buffer CopiedColorsByLOD
						// create array of all white verts
						GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SetInstanceColorDataForLOD(Component, LODIndex, FColor::White, FColor::White);
					}
					else
					{
						FPerLODVertexColorData& LODData = PasteColors->PerLODVertexColorData[LODIndex];
						const int32 NumLODVertices = LodRenderData.GetNumVertices();

						if (NumLODVertices == LODData.ColorsByIndex.Num())
						{
							GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SetInstanceColorDataForLOD(Component, LODIndex, LODData.ColorsByIndex);
						}
						else
						{
							// verts counts mismatch - build translation/fixup list of colors in ReOrderedColors
							TArray<FColor> PositionMatchedColors;
							PositionMatchedColors.Empty(NumLODVertices);

							for (int32 VertexIndex = 0; VertexIndex < NumLODVertices; ++VertexIndex)
							{
								// Search for color matching this vertex position otherwise fill it with white
								const FVector& Vertex = (FVector)LodRenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
								const FColor* FoundColor = LODData.ColorsByPosition.Find(Vertex);
								PositionMatchedColors.Add(FoundColor ? *FoundColor : FColor::White);
							}

							GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->SetInstanceColorDataForLOD(Component, LODIndex, PositionMatchedColors);
						}
					}
				}

				/** Update cached paint data on static mesh component and update DDC key */
				Component->CachePaintedDataIfNecessary();
				Component->StaticMeshDerivedDataKey = Mesh->GetRenderData()->DerivedDataKey;
			}
		}
	}
}

bool UMeshPaintModeSubsystem::CanPasteInstanceVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, const TArray<FPerComponentVertexColorData>& CopiedColorsByComponent)
{
	bool bValidForPasting = false;
	/** Make sure we have copied vertex color data which matches at least mesh component in the current selection */
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		checkf(Component != nullptr, TEXT("Invalid Static Mesh Component"));
		UStaticMesh* Mesh = Component->GetStaticMesh();
		if (Mesh && Mesh->GetNumLODs() > 0)
		{
			// See if there is a valid instance of copied vertex colors for this mesh
			const int32 BlueprintCreatedComponentIndex = Component->GetBlueprintCreatedComponentIndex();
			const FPerComponentVertexColorData* PasteColors = CopiedColorsByComponent.FindByPredicate([=](const FPerComponentVertexColorData& ComponentData)
			{
				return (ComponentData.OriginalMesh.Get() == Mesh && ComponentData.ComponentIndex == BlueprintCreatedComponentIndex);
			});

			if (PasteColors)
			{
				bValidForPasting = true;
				break;
			}
		}
	}

	return bValidForPasting;
}

void UMeshPaintModeSubsystem::RemovePerLODColors(const TArray<UMeshComponent*>& PaintableComponents)
{
	//Remove painting on all lowers LODs before doing the propagation
	for (UMeshComponent* SelectedComponent : PaintableComponents)
	{
		UStaticMeshComponent *StaticMeshComponent = Cast<UStaticMeshComponent>(SelectedComponent);
		if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh())
		{
			// Mark the mesh component as modified
			StaticMeshComponent->Modify();

			// If this is called from the Remove button being clicked the SMC wont be in a Reregister context,
			// but when it gets called from a Paste or Copy to Source operation it's already inside a more specific
			// SMCRecreateScene context so we shouldn't put it inside another one.
			if (StaticMeshComponent->IsRenderStateCreated())
			{
				// Detach all instances of this static mesh from the scene.
				FComponentReregisterContext ComponentReregisterContext(StaticMeshComponent);

				for (int32 LODIndex = 1; LODIndex < StaticMeshComponent->LODData.Num(); ++LODIndex)
				{
					StaticMeshComponent->RemoveInstanceVertexColorsFromLOD(LODIndex);
				}
			}
			else
			{
				for (int32 LODIndex = 1; LODIndex < StaticMeshComponent->LODData.Num(); ++LODIndex)
				{
					StaticMeshComponent->RemoveInstanceVertexColorsFromLOD(LODIndex);
				}
			}
		}
	}
}

bool UMeshPaintModeSubsystem::CanFixTextureColors(const TArray<UMeshComponent*>& Components)
{
	for (UMeshComponent* Component : Components)
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			if (UTexture* Texture = StaticMeshComponent->GetMeshPaintTexture())
			{
				if (StaticMeshComponent->CanMeshPaintTextureColors())
				{
					if (StaticMeshComponent->GetMeshPaintTextureResolution() != Texture->Source.GetSizeX())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void UMeshPaintModeSubsystem::FixTextureColors(const TArray<UMeshComponent*>& Components)
{
	for (UMeshComponent* Component : Components)
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
		{
			if (UTexture* Texture = StaticMeshComponent->GetMeshPaintTexture())
			{
				int32 TextureResolution = StaticMeshComponent->GetMeshPaintTextureResolution();
				if (TextureResolution != Texture->Source.GetSizeX())
				{
					FImage Image;
					if (Texture->Source.GetMipImage(Image, 0))
					{
						FImage ResizedImage(TextureResolution, TextureResolution, Image.NumSlices, Image.Format, Image.GammaSpace);
						FImageCore::ResizeImage(Image, ResizedImage);

						Texture->Modify();
						Texture->Source.Init(ResizedImage);
						Texture->PostEditChange();
					}
				}
			}
		}
	}
}

void UMeshPaintModeSubsystem::SwapColors()
{
	if (UMeshPaintingToolProperties* Settings = UMeshPaintMode::GetToolProperties())
	{
		Settings->Modify();

		FLinearColor TempPaintColor = Settings->PaintColor;
		Settings->PaintColor = Settings->EraseColor;
		Settings->EraseColor = TempPaintColor;
	}
}
