// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPaintingToolsetTypes.h"
#include "UObject/Package.h"
#include "MeshPaintModeSettings.generated.h"

/** Mesh paint color view modes (somewhat maps to EVertexColorViewMode engine enum.) */
UENUM()
enum class EMeshPaintColorView : uint8
{
	/** Normal view mode (vertex color visualization off) */
	Normal UMETA(DisplayName = "Off"),

	/** RGB only */
	RGB UMETA(DisplayName = "RGB Channels"),

	/** Alpha only */
	Alpha UMETA(DisplayName = "Alpha Channel"),

	/** Red only */
	Red UMETA(DisplayName = "Red Channel"),

	/** Green only */
	Green UMETA(DisplayName = "Green Channel"),

	/** Blue only */
	Blue UMETA(DisplayName = "Blue Channel"),
};

/**
 * Implements the Mesh Editor's settings.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UMeshPaintModeSettings
	: public UObject
{
	GENERATED_BODY()

public:
	/** Color view mode used to display Vertex Colors */
	UPROPERTY(config, EditAnywhere, Category = Visualization)
	EMeshPaintDataColorViewMode ColorViewMode = EMeshPaintDataColorViewMode::Normal;

	/** Palette tab to use when entering the tool */
	UPROPERTY(config)
	FName DefaultPalette;
};
