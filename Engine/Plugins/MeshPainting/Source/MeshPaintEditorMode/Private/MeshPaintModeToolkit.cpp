// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintModeToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "MeshPaintMode.h"
#include "InteractiveToolManager.h"

#define LOCTEXT_NAMESPACE "MeshPaintModeToolkit"

FMeshPaintModeToolkit::~FMeshPaintModeToolkit()
{
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.RemoveAll(this);
}

void FMeshPaintModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.AddSP(this, &FMeshPaintModeToolkit::SetActiveToolMessage);
}

FName FMeshPaintModeToolkit::GetToolkitFName() const
{
	return FName( "MeshPaintMode" );
}

FText FMeshPaintModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT( "ToolkitName", "Mesh Paint Mode" );
}

void FMeshPaintModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Add(UMeshPaintMode::MeshPaintMode_VertexColor);
	PaletteNames.Add(UMeshPaintMode::MeshPaintMode_VertexWeights);
	PaletteNames.Add(UMeshPaintMode::MeshPaintMode_TextureColor);
	PaletteNames.Add(UMeshPaintMode::MeshPaintMode_TextureAsset);
}


FText FMeshPaintModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	if (Palette == UMeshPaintMode::MeshPaintMode_VertexColor)
	{
		return LOCTEXT("MeshPaintMode_VertexColor", "Vertex\nColor");
	}
	if (Palette == UMeshPaintMode::MeshPaintMode_VertexWeights)
	{
		return LOCTEXT("MeshPaintMode_VertexWeights", "Vertex\nWeights");
	}
	if (Palette == UMeshPaintMode::MeshPaintMode_TextureColor)
	{
		return LOCTEXT("MeshPaintMode_TextureColor", "Texture\nColor");
	}
	if (Palette == UMeshPaintMode::MeshPaintMode_TextureAsset)
	{
		return LOCTEXT("MeshPaintMode_TextureAsset", "Textures");
	}
	return FText();
}

FText FMeshPaintModeToolkit::GetActiveToolDisplayName() const 
{ 
	if (UInteractiveTool* ActiveTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left))
	{
		return ActiveTool->GetClass()->GetDisplayNameText();
	}

	return LOCTEXT("MeshPaintNoActiveTool", "Mesh Paint");
}

FText FMeshPaintModeToolkit::GetActiveToolMessage() const 
{ 
	if (UInteractiveTool* ActiveTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left))
	{
		return ActiveToolMessageCache;
	}
	return LOCTEXT("MeshPaintDefaultMessage", "Select a mesh.");
}

void FMeshPaintModeToolkit::SetActiveToolMessage(const FText& Message)
{
	ActiveToolMessageCache = Message;	
}

#undef LOCTEXT_NAMESPACE
