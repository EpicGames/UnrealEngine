// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Toolkits/BaseToolkit.h"


/**
 * Mode Toolkit for the Mesh Paint Mode
 */
class FMeshPaintModeToolkit : public FModeToolkit
{
public:

	FMeshPaintModeToolkit(  )
	{}

	~FMeshPaintModeToolkit();

	/** FModeToolkit interface */
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	
	// IToolkit overrides
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const override;
	virtual FText GetToolPaletteDisplayName(FName Palette) const override;

	virtual FText GetActiveToolDisplayName() const;
	virtual FText GetActiveToolMessage() const;

	void SetActiveToolMessage(const FText&);

private:
	FText ActiveToolMessageCache;

};
