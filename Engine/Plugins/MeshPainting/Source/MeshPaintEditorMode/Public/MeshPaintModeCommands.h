// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Tools/InteractiveToolsCommands.h"
#include "Tools/StandardToolModeCommands.h"


class FMeshPaintEditorModeCommands : public TCommands<FMeshPaintEditorModeCommands>
{
public:
	FMeshPaintEditorModeCommands() : TCommands<FMeshPaintEditorModeCommands>
		(
			"MeshPaint",
			NSLOCTEXT("MeshPaintEditorMode", "MeshPaintingModeCommands", "Mesh Painting Mode"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
			)
	{}

	/**
	* Initialize commands
	*/
	virtual void RegisterCommands() override;

	static TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetCommands()
	{
		return FMeshPaintEditorModeCommands::Get().Commands;
	}

public:
	TSharedPtr<FUICommandInfo> SelectVertex;
	TSharedPtr<FUICommandInfo> SelectTextureColor;
	TSharedPtr<FUICommandInfo> SelectTextureAsset;

	TSharedPtr<FUICommandInfo> PaintVertexColor;
	TSharedPtr<FUICommandInfo> PaintVertexWeight;
	TSharedPtr<FUICommandInfo> PaintTextureColor;
	TSharedPtr<FUICommandInfo> PaintTextureAsset;

	TSharedPtr<FUICommandInfo> SwapColor;
	TSharedPtr<FUICommandInfo> FillVertex;
	TSharedPtr<FUICommandInfo> FillTexture;
	TSharedPtr<FUICommandInfo> PropagateMesh;
	TSharedPtr<FUICommandInfo> PropagateLODs;
	TSharedPtr<FUICommandInfo> SaveVertex;
	TSharedPtr<FUICommandInfo> SaveTexture;
	TSharedPtr<FUICommandInfo> Add;
	TSharedPtr<FUICommandInfo> RemoveVertex;
	TSharedPtr<FUICommandInfo> RemoveTexture;
	TSharedPtr<FUICommandInfo> Copy;
	TSharedPtr<FUICommandInfo> Paste;
	TSharedPtr<FUICommandInfo> Import;
	TSharedPtr<FUICommandInfo> GetTextureColors;
	TSharedPtr<FUICommandInfo> GetVertexColors;
	TSharedPtr<FUICommandInfo> FixVertex;
	TSharedPtr<FUICommandInfo> FixTexture;

	TSharedPtr<FUICommandInfo> PreviousLOD;
	TSharedPtr<FUICommandInfo> NextLOD;
	TSharedPtr<FUICommandInfo> PreviousTexture;
	TSharedPtr<FUICommandInfo> NextTexture;

	TSharedPtr<FUICommandInfo> IncreaseBrushRadius;
	TSharedPtr<FUICommandInfo> DecreaseBrushRadius;
	TSharedPtr<FUICommandInfo> IncreaseBrushStrength;
	TSharedPtr<FUICommandInfo> DecreaseBrushStrength;
	TSharedPtr<FUICommandInfo> IncreaseBrushFalloff;
	TSharedPtr<FUICommandInfo> DecreaseBrushFalloff;

	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> Commands;
};


class FMeshPaintingToolActionCommands : public TInteractiveToolCommands<FMeshPaintingToolActionCommands>
{
public:
	FMeshPaintingToolActionCommands() :
		TInteractiveToolCommands<FMeshPaintingToolActionCommands>(
			"MeshPaintingTools", // Context name for fast lookup
			NSLOCTEXT("MeshPaintEditorMode", "MeshPaintingToolsCommands", "Mesh Painting Tools"), // Localized context name for displaying
			NAME_None, // Parent
			FAppStyle::GetAppStyleSetName() // Icon Style Set
			)
	{
	}

	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;


	/**
	 * interface that hides various per-tool action sets
	 */

	 /**
	  * Register all Tool command sets. Call this in module startup
	  */
	static void RegisterAllToolActions();

	/**
	 * Unregister all Tool command sets. Call this from module shutdown.
	 */
	static void UnregisterAllToolActions();

	/**
	 * Add or remove commands relevant to Tool to the given UICommandList.
	 * Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	 * @param bUnbind if true, commands are removed, otherwise added
	 */
	static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);
};
