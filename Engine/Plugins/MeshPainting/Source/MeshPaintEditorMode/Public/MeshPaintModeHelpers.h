// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "MeshPaintModeHelpers.generated.h"

#define UE_API MESHPAINTEDITORMODE_API

class FMeshPaintParameters;
class UImportVertexColorOptions;
class UTexture2D;
class UStaticMeshComponent;
class UStaticMesh;
class USkeletalMesh;
class IMeshPaintComponentAdapter;
class UPaintBrushSettings;
class FEditorViewportClient;
class UMeshComponent;
class USkeletalMeshComponent;
class UViewportInteractor;
class FViewport;
class FPrimitiveDrawInterface;
class FSceneView;
class UInteractiveTool;

struct FStaticMeshComponentLODInfo;
struct FPerComponentVertexColorData;

UENUM()
enum class EMeshPaintActiveMode : uint8
{
	VertexColor UMETA(DisplayName = "VertexColor"),
	VertexWeights UMETA(DisplayName = "VertexWeights"),
	TextureColor UMETA(DisplayName = "TextureColor"),
	Texture UMETA(DisplayName = "Texture"),
};

enum class EMeshPaintDataColorViewMode : uint8;

class UMeshPaintModeSubsystem : public UEditorSubsystem
{
public:
	/** Forces the Viewport Client to render using the given Viewport Color ViewMode */
	UE_API void SetViewportColorMode(EMeshPaintActiveMode ActiveMode, EMeshPaintDataColorViewMode ColorViewMode, FEditorViewportClient* ViewportClient, UInteractiveTool const* ActiveTool);

	/** Sets whether or not the viewport should be real time rendered */
	UE_API void SetRealtimeViewport(FEditorViewportClient* ViewportClient, bool bRealtime);


	/** Helper function to import Vertex Colors from a Texture to the specified MeshComponent (makes use of SImportVertexColorsOptions Widget) */
	UE_API void ImportVertexColorsFromTexture(UMeshComponent* MeshComponent);

	/** Imports vertex colors from a Texture to the specified Skeletal Mesh according to user-set options */
	UE_API void ImportVertexColorsToSkeletalMesh(USkeletalMesh* SkeletalMesh, const UImportVertexColorOptions* Options, UTexture2D* Texture);

	/** Helper function to import Vertex Colors from a the MeshPaintTexture on the mesh component */
	UE_API void ImportVertexColorsFromMeshPaintTexture(UMeshComponent* MeshComponent);

	/** Helper function to import the MeshPaintTexture on the mesh component from the vertex colors */
	UE_API void ImportMeshPaintTextureFromVertexColors(UMeshComponent* MeshComponent);

	struct FPaintRay
	{
		FVector CameraLocation;
		FVector RayStart;
		FVector RayDirection;
		UViewportInteractor* ViewportInteractor;
	};


	UE_API bool RetrieveViewportPaintRays(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI, TArray<FPaintRay>& OutPaintRays);

	/** Imports vertex colors from a Texture to the specified Static Mesh according to user-set options */
	UE_API void ImportVertexColorsToStaticMesh(UStaticMesh* StaticMesh, const UImportVertexColorOptions* Options, UTexture2D* Texture);

	/** Imports vertex colors from a Texture to the specified Static Mesh Component according to user-set options */
	UE_API void ImportVertexColorsToStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent, const UImportVertexColorOptions* Options, UTexture2D* Texture);

	UE_API void PropagateVertexColors(const TArray<UStaticMeshComponent *> StaticMeshComponents);
	UE_API bool CanPropagateVertexColors(TArray<UStaticMeshComponent*>& StaticMeshComponents, TArray<UStaticMesh*>& StaticMeshes, int32 NumInstanceVertexColorBytes);
	UE_API void CopyVertexColors(const TArray<UStaticMeshComponent*> StaticMeshComponents, TArray<FPerComponentVertexColorData>& CopiedVertexColors);
	UE_API bool CanCopyInstanceVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, int32 PaintingMeshLODIndex);
	UE_API void PasteVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, TArray<FPerComponentVertexColorData>& CopiedColorsByComponent);
	UE_API bool CanPasteInstanceVertexColors(const TArray<UStaticMeshComponent*>& StaticMeshComponents, const TArray<FPerComponentVertexColorData>& CopiedColorsByComponent);
	UE_API void RemovePerLODColors(const TArray<UMeshComponent*>& PaintableComponents);
	
	UE_API bool CanFixTextureColors(const TArray<UMeshComponent*>& Components);
	UE_API void FixTextureColors(const TArray<UMeshComponent*>& Components);

	UE_API void SwapColors();
};

#undef UE_API
