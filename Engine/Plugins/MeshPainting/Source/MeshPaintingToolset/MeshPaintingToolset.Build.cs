// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MeshPaintingToolset : ModuleRules
	{
        public MeshPaintingToolset(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
					"RenderCore",
					"RHI",
					"MeshDescription",
					"StaticMeshDescription",
					"GeometryCollectionEngine",
					"TypedElementFramework",
					"TypedElementRuntime",
					"UnrealEd",
					"InterchangeEngine",
					"InterchangePipelines",
					"Chaos",
					"ImageCore"
				}
                );

                PublicDependencyModuleNames.AddRange(
                new string[] {
					"InteractiveToolsFramework",
					"EditorInteractiveToolsFramework",
					"GeometryCore",
                }
            );
        }
    }
}
