// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseMeshPaintComponentAdapter.h"
#include "Distance/DistLine3Segment3.h"
#include "Distance/DistLine3Triangle3.h"
#include "Distance/DistSegment3Triangle3.h"

bool FBaseMeshPaintComponentAdapter::Initialize()
{
	bool bInit = InitializeVertexData() && BuildOctree();
	FIndexMeshArrayAdapterd TempAdapter(&MeshVertices, &MeshIndices);
	Adapter = TempAdapter;
	AABBTree = MakeUnique<UE::Geometry::TMeshAABBTree3<const FIndexMeshArrayAdapterd>>(&Adapter);
	return bInit;
}

bool FBaseMeshPaintComponentAdapter::BuildOctree()
{
	bool bValidOctree = false;

	if (MeshVertices.Num() > 0 && MeshIndices.Num() > 0)
	{
		bValidOctree = true;
		
		// First determine bounding box of mesh verts
		FBox Bounds(ForceInit);
		for (const FVector& Vertex : MeshVertices)
		{
			Bounds += Vertex;
		}

		MeshTriOctree = MakeUnique<FMeshPaintTriangleOctree>(Bounds.GetCenter(), Bounds.GetExtent().GetMax());

		const uint32 NumIndexBufferIndices = MeshIndices.Num();
		check(NumIndexBufferIndices % 3 == 0);
		const uint32 NumTriangles = NumIndexBufferIndices / 3;

		for (uint32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
		{
			// Grab the vertex indices and points for this triangle
			FMeshPaintTriangle MeshTri;
			MeshTri.Vertices[0] = MeshVertices[MeshIndices[TriIndex * 3 + 0]];
			MeshTri.Vertices[1] = MeshVertices[MeshIndices[TriIndex * 3 + 1]];
			MeshTri.Vertices[2] = MeshVertices[MeshIndices[TriIndex * 3 + 2]];
			MeshTri.Normal = FVector::CrossProduct(MeshTri.Vertices[1] - MeshTri.Vertices[0], MeshTri.Vertices[2] - MeshTri.Vertices[0]).GetSafeNormal();
			MeshTri.Index = TriIndex;

			FBox TriBox;
			TriBox.Min.X = FMath::Min3(MeshTri.Vertices[0].X, MeshTri.Vertices[1].X, MeshTri.Vertices[2].X);
			TriBox.Min.Y = FMath::Min3(MeshTri.Vertices[0].Y, MeshTri.Vertices[1].Y, MeshTri.Vertices[2].Y);
			TriBox.Min.Z = FMath::Min3(MeshTri.Vertices[0].Z, MeshTri.Vertices[1].Z, MeshTri.Vertices[2].Z);

			TriBox.Max.X = FMath::Max3(MeshTri.Vertices[0].X, MeshTri.Vertices[1].X, MeshTri.Vertices[2].X);
			TriBox.Max.Y = FMath::Max3(MeshTri.Vertices[0].Y, MeshTri.Vertices[1].Y, MeshTri.Vertices[2].Y);
			TriBox.Max.Z = FMath::Max3(MeshTri.Vertices[0].Z, MeshTri.Vertices[1].Z, MeshTri.Vertices[2].Z);

			MeshTri.BoxCenterAndExtent = FBoxCenterAndExtent(TriBox);
			MeshTriOctree->AddElement(MeshTri);
		}
	}

	return bValidOctree;	
}

const TArray<FVector>& FBaseMeshPaintComponentAdapter::GetMeshVertices() const
{
	return MeshVertices;
}

const TArray<uint32>& FBaseMeshPaintComponentAdapter::GetMeshIndices() const
{
	return MeshIndices;
}

void FBaseMeshPaintComponentAdapter::GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const
{
	OutVertex = MeshVertices[VertexIndex];
}

TArray<uint32> FBaseMeshPaintComponentAdapter::SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const
{
	TArray<uint32> OutTriangles;

	// Use a bit of distance bias to make sure that we get all of the overlapping triangles.  We
	// definitely don't want our brush to be cut off by a hard triangle edge
	const float SquaredRadiusBias = ComponentSpaceSquaredBrushRadius * 0.025f;

	MeshTriOctree->FindElementsWithBoundsTest(FBoxCenterAndExtent(ComponentSpaceBrushPosition, FVector(FMath::Sqrt(ComponentSpaceSquaredBrushRadius + SquaredRadiusBias))), [&OutTriangles, bOnlyFrontFacing, &ComponentSpaceCameraPosition](const FMeshPaintTriangle& CurrentTri)
	{
		const float SignedPlaneDist = FVector::PointPlaneDist(ComponentSpaceCameraPosition, CurrentTri.Vertices[0], CurrentTri.Normal);
		if (!bOnlyFrontFacing || SignedPlaneDist < 0.0f)
		{
			OutTriangles.Add(CurrentTri.Index);
		}
	});

	return OutTriangles;
}

void FBaseMeshPaintComponentAdapter::GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const
{
	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	const TArray<uint32> InfluencedTriangles = SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		bOnlyFrontFacing);

	// Make sure we're dealing with triangle lists
	const int32 NumIndexBufferIndices = MeshIndices.Num();
	check(NumIndexBufferIndices % 3 == 0);

	InfluencedVertices.Reserve(InfluencedTriangles.Num());
	for (const int32 InfluencedTriangle : InfluencedTriangles)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const FVector& VertexPosition = MeshVertices[MeshIndices[InfluencedTriangle * 3 + Index]];
			if ((VertexPosition - ComponentSpaceBrushPosition).SizeSquared() <= ComponentSpaceSquaredBrushRadius)
			{
				InfluencedVertices.Add(MeshIndices[InfluencedTriangle * 3 + Index]);
			}
		}
	}
}

void FBaseMeshPaintComponentAdapter::GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const
{
	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	TArray<uint32> InfluencedTriangles = SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		bOnlyFrontFacing);

	// Make sure we're dealing with triangle lists
	const int32 NumIndexBufferIndices = MeshIndices.Num();
	check(NumIndexBufferIndices % 3 == 0);

	OutData.Reserve(InfluencedTriangles.Num() * 3);
	for(int32 InfluencedTriangle : InfluencedTriangles)
	{
		for(int32 Index = 0; Index < 3; ++Index)
		{
			const FVector& VertexPosition = MeshVertices[MeshIndices[InfluencedTriangle * 3 + Index]];
			if((VertexPosition - ComponentSpaceBrushPosition).SizeSquared() <= ComponentSpaceSquaredBrushRadius)
			{
				OutData.AddDefaulted();
				TPair<int32, FVector>& OutPair = OutData.Last();
				OutPair.Key = MeshIndices[InfluencedTriangle * 3 + Index];
				OutPair.Value = MeshVertices[OutPair.Key];
			}
		}
	}
}

TArray<FVector> FBaseMeshPaintComponentAdapter::SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const
{
	// Get list of intersecting triangles with given sphere data
	const TArray<uint32> IntersectedTriangles = SphereIntersectTriangles(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, bOnlyFrontFacing);
	// Get a list of unique vertices indexed by the influenced triangles
	TSet<int32> InfluencedVertices;
	InfluencedVertices.Reserve(IntersectedTriangles.Num());
	for (int32 IntersectedTriangle : IntersectedTriangles)
	{
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 0]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 1]);
		InfluencedVertices.Add(MeshIndices[IntersectedTriangle * 3 + 2]);
	}

	TArray<FVector> InRangeVertices;
	InRangeVertices.Empty(MeshVertices.Num());
	for (int32 VertexIndex : InfluencedVertices)
	{
		const FVector& Vertex = MeshVertices[VertexIndex];
		if (FVector::DistSquared(ComponentSpaceBrushPosition, Vertex) <= ComponentSpaceSquaredBrushRadius)
		{
			InRangeVertices.Add(Vertex);
		}
	}

	return InRangeVertices;
}

bool FBaseMeshPaintComponentAdapter::RayIntersectAdapter(UE::Geometry::FIndex3i& HitTriangle, FVector& HitPosition, const FVector Start, const FVector End) const
{
	int32 HitTriangleID;
	double NearestT;
	FVector3d Direction((FVector3d)End - (FVector3d)Start);
	UE::Geometry::Normalize(Direction);
	FRay3d Ray((FVector3d)Start, Direction);
	bool bHit = AABBTree->FindNearestHitTriangle(Ray, NearestT, HitTriangleID);
	if (bHit)
	{
		HitPosition = (FVector)Ray.PointAt(NearestT);
		HitTriangle = Adapter.GetTriangle(HitTriangleID);
	}
	return bHit;
}
