// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshPaintComponentAdapter.h"

#include "Components/MeshComponent.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialInstance.h"
#include "MeshPaintingToolsetTypes.h"
#include "ObjectCacheContext.h"
#include "TexturePaintToolset.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

//////////////////////////////////////////////////////////////////////////
// IMeshPaintGeometryAdapter

static bool IsTextureSuitableForTexturePainting(const TWeakObjectPtr<UTexture> TexturePtr)
{
	return (TexturePtr.Get() != nullptr &&
		!TexturePtr->IsNormalMap() &&
		!TexturePtr->HasHDRSource() && // Currently HDR textures are not supported to paint on.
		TexturePtr->Source.IsValid() &&
		TexturePtr->Source.GetFormat() != TSF_G16 && // Currently 16 bit textures are not supported to paint on.
		TexturePtr->Source.GetBytesPerPixel() > 0 && // Textures' sources must have a known count of bytes per pixel
		(TexturePtr->Source.GetBytesPerPixel() <= UTexturePaintToolset::GetMaxSupportedBytesPerPixelForPainting())); // Textures' sources must fit in FColor struct to be supported.
}

static void AddPaintableTextureFromMaterialExpression(UMaterialInterface const* InMaterial, UMaterialExpressionTextureBase const* InExpression, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	if (InExpression == nullptr || !IsTextureSuitableForTexturePainting(InExpression->Texture))
	{
		return;
	}

	// Default UV channel to index 0. 
	FPaintableTexture PaintableTexture(InExpression->Texture, 0);

	// Texture Samples can have UV's specified, check the first node for whether it has a custom UV channel set. 
	// We only check the first as the Mesh paint mode does not support painting with UV's modified in the shader.
	if (UMaterialExpressionTextureSample const* TextureSampleExpression = Cast<UMaterialExpressionTextureSample>(InExpression))
	{
		if (UMaterialExpressionTextureCoordinate const* TextureCoordsExpression = Cast<UMaterialExpressionTextureCoordinate>(TextureSampleExpression->Coordinates.Expression))
		{
			// Store the uv channel, this is set when the texture is selected. 
			PaintableTexture.UVChannelIndex = TextureCoordsExpression->CoordinateIndex;
		}
		else
		{
			PaintableTexture.UVChannelIndex = TextureSampleExpression->ConstCoordinate;
		}

		// Handle texture parameter expressions.
		if (UMaterialExpressionTextureSampleParameter const* TextureSampleParameterExpression = Cast<UMaterialExpressionTextureSampleParameter>(TextureSampleExpression))
		{
			// Grab the overridden texture if it exists.  
			InMaterial->GetTextureParameterValue(TextureSampleParameterExpression->ParameterName, PaintableTexture.Texture);
		}
	}

	// Note that the same texture will be added again if its UV channel differs. 
	const int32 TextureIndex = InOutTextureList.AddUnique(PaintableTexture);

	// Cache the first default index, if there is no previous info this will be used as the selected texture.
	if ((OutDefaultIndex == INDEX_NONE) && InExpression->IsDefaultMeshpaintTexture)
	{
		OutDefaultIndex = TextureIndex;
	}
}

void IMeshPaintComponentAdapter::DefaultQueryPaintableTextures(int32 MaterialIndex, const UMeshComponent* MeshComponent, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	OutDefaultIndex = INDEX_NONE;

	// We already know the material we are painting on, take it off the static mesh component
	UMaterialInterface const* Material = MeshComponent->GetMaterial(MaterialIndex);

	while (Material != nullptr)
	{
		if (Material != NULL)
		{
			// First iterate top level material expressions.
			for (UMaterialExpression* MaterialExpression : Material->GetMaterial()->GetExpressions())
			{
				if (UMaterialExpressionTextureBase* TextureExpression = Cast<UMaterialExpressionTextureBase>(MaterialExpression))
				{
					AddPaintableTextureFromMaterialExpression(Material, TextureExpression, OutDefaultIndex, InOutTextureList);
				}
			}

			// Now iterate material expressions from material functions.
			TArray<UMaterialFunctionInterface*> MaterialFunctions;
			Material->GetDependentFunctions(MaterialFunctions);
			for (UMaterialFunctionInterface* MaterialFunction : MaterialFunctions)
			{
				for (UMaterialExpression* MaterialExpression : MaterialFunction->GetExpressions())
				{
					if (UMaterialExpressionTextureBase* TextureExpression = Cast<UMaterialExpressionTextureBase>(MaterialExpression))
					{
						AddPaintableTextureFromMaterialExpression(Material, TextureExpression, OutDefaultIndex, InOutTextureList);
					}
				}
			}
		}
		
		// Make sure to include all texture parameters.
		TMap<FMaterialParameterInfo, FMaterialParameterMetadata> ParameterValues;
		Material->GetAllParametersOfType(EMaterialParameterType::Texture, ParameterValues);

		for (auto& ParameterElem : ParameterValues)
		{
			const TWeakObjectPtr<UTexture> TexturePtr = ParameterElem.Value.Value.Texture;

			if (IsTextureSuitableForTexturePainting(TexturePtr))
			{
				FPaintableTexture PaintableTexture;

				// Default UV channel to index 0.
				PaintableTexture = FPaintableTexture(TexturePtr.Get(), 0);
				InOutTextureList.AddUnique(PaintableTexture);
			}
		}

		if (UMaterialInstance const* MaterialInstance = Cast<UMaterialInstance>(Material))
		{
			Material = MaterialInstance->Parent ? Cast<UMaterialInstance>(MaterialInstance->Parent) : nullptr;
		}
		else
		{
			// This prevents an infinite loop when `Material` isn't a material instance.
			break;
		}
	}

	// If the component has a mesh paint texture, then add it here.
	if (UTexture* MeshPaintTexture = MeshComponent->GetMeshPaintTexture())
	{
		const int32 CoordinateIndex = MeshComponent->GetMeshPaintTextureCoordinateIndex();
		InOutTextureList.AddUnique(FPaintableTexture(MeshPaintTexture, CoordinateIndex, true));
	}
}

void IMeshPaintComponentAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const
{
	FMaterialUpdateContext MaterialUpdateContext;
	ApplyOrRemoveTextureOverride(SourceTexture, OverrideTexture, MaterialUpdateContext);
}

namespace UE::MeshPaintingToolset
{
	namespace Private
	{
		struct FOverrideData
		{
			TWeakObjectPtr<UTexture> OverrideTexture;
			TSet<ERHIFeatureLevel::Type, DefaultKeyFuncs<ERHIFeatureLevel::Type>, TInlineSetAllocator<1>> OverridenFeatureLevels;
			uint32 Count = 0;
		};

		struct FGlobalTextureOverrideState
		{
			using FOverrideKey = TPair<TWeakObjectPtr<UMaterialInterface>, TWeakObjectPtr<const UTexture>>;

			static void DuplicateOverrideOwnership(const FDefaultTextureOverride* Current, const FDefaultTextureOverride* To)
			{
				TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>>* Overrides = DefaultTextureOverrideToOverrides.Find(Current);
				if (Overrides)
				{
					for (const FOverrideKey& Pair : *Overrides)
					{
						OverridesData.Find(Pair)->Count += 1;
					}

					// Duplicate the override in case an allocation change the addresses
					DefaultTextureOverrideToOverrides.Add(To, TSet<FOverrideKey>(*Overrides));
				}
			}

			static void TransferOverrideOwnership(const FDefaultTextureOverride* Current, const FDefaultTextureOverride* To)
			{
				TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>> Overrides;
				if (DefaultTextureOverrideToOverrides.RemoveAndCopyValue(Current, Overrides))
				{
					DefaultTextureOverrideToOverrides.Add(To, MoveTemp(Overrides));
				}
			}

			static void RegisterMaterialOverride(const FDefaultTextureOverride* Requester, UMaterialInterface* Material, const UTexture* SourceTexture, UTexture* OverrrideTexture, const ERHIFeatureLevel::Type FeatureLevel, bool& bOutUpdated)
			{
				TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>>& Overrides = DefaultTextureOverrideToOverrides.FindOrAdd(Requester);
				
				FOverrideKey Pair = FOverrideKey(TWeakObjectPtr<UMaterialInterface>(Material), TWeakObjectPtr<const UTexture>(SourceTexture));
				uint32 PairHash = GetTypeHash(Pair);

				Overrides.AddByHash(PairHash, Pair);
				
				FOverrideData& OverrideData = OverridesData.FindOrAddByHash(PairHash, Pair);
				
				bool bFeatureLevelAlreadyOverriden = false;
				uint32 FeatureLevelHash = GetTypeHash(FeatureLevel);
				OverrideData.OverridenFeatureLevels.AddByHash(FeatureLevelHash, FeatureLevel, &bFeatureLevelAlreadyOverriden);

				if (OverrideData.Count == 0 || OverrideData.OverrideTexture != OverrrideTexture)
				{
					OverrideData.OverrideTexture = OverrrideTexture;

					for (const ERHIFeatureLevel::Type LevelToUpdate : OverrideData.OverridenFeatureLevels)
					{
						Material->OverrideTexture(SourceTexture, OverrrideTexture, LevelToUpdate);
						bOutUpdated = true;
					}

					AddMaterialTracking(Pair);
				}
				else if (!bFeatureLevelAlreadyOverriden)
				{
					OverrideData.OverridenFeatureLevels.AddByHash(FeatureLevelHash, FeatureLevel);
					Material->OverrideTexture(SourceTexture, OverrrideTexture, FeatureLevel);
				}
	
				OverrideData.Count += 1;
			}

			static void RemoveMaterialOverride(const FDefaultTextureOverride* Requester, UMaterialInterface* Material, const UTexture* SourceTexture, bool& bOutUpdated)
			{
				if (TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>>* Override = DefaultTextureOverrideToOverrides.Find(Requester))
				{
					FOverrideKey Pair = FOverrideKey(TWeakObjectPtr<UMaterialInterface>(Material), TWeakObjectPtr<const UTexture>(SourceTexture));
					uint32 PairHash = GetTypeHash(Pair);

					if (Override->RemoveByHash(PairHash, Pair) > 0)
					{
						if (FOverrideData* OverrideData = OverridesData.FindByHash(PairHash, Pair))
						{
							OverrideData->Count -= 1;
							if (OverrideData->Count == 0)
							{
								for (const ERHIFeatureLevel::Type FeatureLevel : OverrideData->OverridenFeatureLevels)
								{
									Material->OverrideTexture(SourceTexture, nullptr, FeatureLevel);
									bOutUpdated = true;
								}

								OverridesData.RemoveByHash(PairHash, Pair);
								RemoveMaterialTracking(Pair);
							}
						}
					}
				}
			}

			static void FreeOverrideOwnership(const FDefaultTextureOverride* Requester)
			{
				TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>> Overrides;
				if (DefaultTextureOverrideToOverrides.RemoveAndCopyValue(Requester, Overrides))
				{
					for (const FOverrideKey& Override : Overrides)
					{
						uint32 OverrideHash = GetTypeHash(Override);
						if (FOverrideData* OverrideData = OverridesData.FindByHash(OverrideHash, Override))
						{
							OverrideData->Count -= 1;
							if (OverrideData->Count == 0)
							{
								if (UMaterialInterface* Material = Override.Key.Get())
								{
									if (const UTexture* SourceTexture = Override.Value.Get())
									{
										for (const ERHIFeatureLevel::Type LevelToUpdate : OverrideData->OverridenFeatureLevels)
										{
											Material->OverrideTexture(SourceTexture, nullptr, LevelToUpdate);
										}
									}
								}

								OverridesData.RemoveByHash(OverrideHash, Override);
								RemoveMaterialTracking(Override);
							}
						}
					}
				}
			}

			static void AddMaterialTracking(const FOverrideKey& Override)
			{
				if (MaterialsAndTexturesOverriden.IsEmpty())
				{
					OnObjectModifiedDelegateHandle = FCoreUObjectDelegates::OnObjectModified.AddStatic(&FGlobalTextureOverrideState::OnObjectModified);
					PostEditDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddStatic(&FGlobalTextureOverrideState::OnObjectPropertyChanged);
				}

				TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>& Textures = MaterialsAndTexturesOverriden.FindOrAdd(Override.Key);
				Textures.Add(Override.Value);
			}

			static void RemoveMaterialTracking(const FOverrideKey& Override)
			{
				uint32 MaterialHash = GetTypeHash(Override.Key);
				if (TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>* Textures = MaterialsAndTexturesOverriden.FindByHash(MaterialHash, Override.Key))
				{
					Textures->Remove(Override.Value);
					if (Textures->IsEmpty())
					{
						MaterialsAndTexturesOverriden.RemoveByHash(MaterialHash, Override.Key);
						if (MaterialsAndTexturesOverriden.IsEmpty())
						{
							FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedDelegateHandle);
							FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PostEditDelegateHandle);
						}
					}
				}
			}

			static void OnObjectModified(UObject* Object)
			{
				if (UMaterialInterface* Material = Cast<UMaterialInterface>(Object))
				{
					FOverrideKey Override;
					Override.Key = Material;
					if (const TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>* Textures = MaterialsAndTexturesOverriden.Find(Override.Key))
					{
						for (const TWeakObjectPtr<const UTexture>& Texture : *Textures)
						{
							if (const UTexture* RawTexturePtr = Texture.Get())
							{
								Override.Value = Texture;
								if (FOverrideData* OverrideData = OverridesData.Find(Override))
								{
									for (const ERHIFeatureLevel::Type FeatureLevel : OverrideData->OverridenFeatureLevels)
									{
										// The material resource might change because of the modifications. To avoid leaking some temp texture overrides, this just remove the temporary overrides during the modification.
										Material->OverrideTexture(RawTexturePtr, nullptr, FeatureLevel);
									}
								}
							}
						}
					}
				}
			}

			static void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
			{
				if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
				{
					if (UMaterialInterface* Material = Cast<UMaterialInterface>(Object))
					{
						FOverrideKey Override;
						Override.Key = Material;
						if (const TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>* Textures = MaterialsAndTexturesOverriden.Find(Override.Key))
						{
							for (const TWeakObjectPtr<const UTexture>& Texture : *Textures)
							{
								if (const UTexture* RawTexturePtr = Texture.Get())
								{
									Override.Value = Texture;
									if (FOverrideData* OverrideData = OverridesData.Find(Override))
									{
										if (UTexture* OverrideTexture = OverrideData->OverrideTexture.Get())
										{ 
											for (const ERHIFeatureLevel::Type FeatureLevel : OverrideData->OverridenFeatureLevels)
											{
												// Reapply the temporary overrides after the modification.
												Material->OverrideTexture(RawTexturePtr, OverrideTexture, FeatureLevel);
											}
										}
									}
								}
							}
						}
					}
				}
			}


		private:
			static TMap<FOverrideKey, FOverrideData> OverridesData;
			static TMap<const FDefaultTextureOverride*, TSet<FOverrideKey, DefaultKeyFuncs<FOverrideKey>, TInlineSetAllocator<2>>> DefaultTextureOverrideToOverrides;
			static TMap<TWeakObjectPtr<UMaterialInterface>, TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>> MaterialsAndTexturesOverriden;

			static FDelegateHandle OnObjectModifiedDelegateHandle;
			static FDelegateHandle PostEditDelegateHandle;
		};

		TMap<FGlobalTextureOverrideState::FOverrideKey, FOverrideData> FGlobalTextureOverrideState::OverridesData;
		TMap<const FDefaultTextureOverride*, TSet<FGlobalTextureOverrideState::FOverrideKey, DefaultKeyFuncs<FGlobalTextureOverrideState::FOverrideKey>, TInlineSetAllocator<2>>> FGlobalTextureOverrideState::DefaultTextureOverrideToOverrides;
		TMap<TWeakObjectPtr<UMaterialInterface>, TSet<TWeakObjectPtr<const UTexture>, DefaultKeyFuncs<TWeakObjectPtr<const UTexture>>, TInlineSetAllocator<2>>> FGlobalTextureOverrideState::MaterialsAndTexturesOverriden;
		FDelegateHandle FGlobalTextureOverrideState::OnObjectModifiedDelegateHandle;
		FDelegateHandle FGlobalTextureOverrideState::PostEditDelegateHandle;
	}


	FDefaultTextureOverride::FDefaultTextureOverride(const FDefaultTextureOverride& InOther)
	{
		operator=(InOther);
	}

	FDefaultTextureOverride::FDefaultTextureOverride(FDefaultTextureOverride&& InOther)
	{
		operator=(MoveTemp(InOther));
	}

	FDefaultTextureOverride& FDefaultTextureOverride::operator=(const FDefaultTextureOverride& InOther)
	{
		Private::FGlobalTextureOverrideState::DuplicateOverrideOwnership(this, &InOther);
		return *this;
	}

	FDefaultTextureOverride& FDefaultTextureOverride::operator=(FDefaultTextureOverride&& InOther)
	{
		Private::FGlobalTextureOverrideState::TransferOverrideOwnership(this, &InOther);
		return *this;
	}

	void FDefaultTextureOverride::ApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const
	{
		check(IsInGameThread());

		const ERHIFeatureLevel::Type FeatureLevel = InMeshComponent->GetWorld()->GetFeatureLevel();

		TSet<UMaterialInterface*> MaterialsToOverride;

		// Applying the texture override to all materials shows live update throughout the scene not just on painted component.
		FObjectCacheContextScope ObjectCacheScope;
		for (UMaterialInterface* Material : ObjectCacheScope.GetContext().GetMaterialsAffectedByTexture(SourceTexture))
		{
			// Texture overrides are stored on the static resources so walk to first parent with static permutation.
			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
			while (MaterialInstance)
			{
				if (MaterialInstance->bHasStaticPermutationResource)
				{
					break;
				}
				MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
			}
			if (MaterialInstance)
			{
				MaterialsToOverride.Add(MaterialInstance);
			}
			else if (UMaterial* BaseMaterial = Material->GetMaterial())
			{
				MaterialsToOverride.Add(BaseMaterial);
			}
		}

		for (UMaterialInterface* Material : MaterialsToOverride)
		{
			bool bMaterialUpdated = false;
			
			if (!OverrideTexture)
			{
				// Unregister for all materials. This will not affect the materials that weren't overridden by this instance
				Private::FGlobalTextureOverrideState::RemoveMaterialOverride(this, Material, SourceTexture, bMaterialUpdated);
			}
			else
			{
				// Override and register to track modified materials.
				Private::FGlobalTextureOverrideState::RegisterMaterialOverride(this, Material, SourceTexture, OverrideTexture, FeatureLevel, bMaterialUpdated);
			}

			if (bMaterialUpdated)
			{
				MaterialUpdateContext.AddMaterialInterface(Material);
			}
		}

		// Check to see if the source texture is the special mesh paint texture on the component.
		// But always apply setting override to nullptr which can happen after the SourceTexture is cleared from the component.
		if (InMeshComponent->GetMeshPaintTexture() == SourceTexture || !OverrideTexture)
		{
			InMeshComponent->SetMeshPaintTextureOverride(OverrideTexture);
		}
	}

	void FDefaultTextureOverride::ApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture) const
	{
		FMaterialUpdateContext MaterialUpdateContext;
		ApplyOrRemoveTextureOverride(InMeshComponent, SourceTexture, OverrideTexture, MaterialUpdateContext);
	}


	FDefaultTextureOverride::~FDefaultTextureOverride()
	{
		check(IsInGameThread());

		Private::FGlobalTextureOverrideState::FreeOverrideOwnership(this);
	}
}
