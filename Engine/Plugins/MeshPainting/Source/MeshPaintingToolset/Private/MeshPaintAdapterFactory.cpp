// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintAdapterFactory.h"
#include "MeshPaintComponentAdapterFactory.h"

TArray<TSharedPtr<class IMeshPaintComponentAdapterFactory>> FMeshPaintComponentAdapterFactory::FactoryList;

TSharedPtr<class IMeshPaintComponentAdapter> FMeshPaintComponentAdapterFactory::CreateAdapterForMesh(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex)
{
	TSharedPtr<IMeshPaintComponentAdapter> Result;

	for (const auto& Factory : FactoryList)
	{
		Result = Factory->Construct(InComponent, InPaintingMeshLODIndex);

		if (Result.IsValid())
		{
			break;
		}
	}

	return Result;
}

void FMeshPaintComponentAdapterFactory::InitializeAdapterGlobals()
{
	for (const auto& Factory : FactoryList)
	{
		Factory->InitializeAdapterGlobals();
	}
}

void FMeshPaintComponentAdapterFactory::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
	for (const auto& Factory : FactoryList)
	{
		Factory->AddReferencedObjectsGlobals(Collector);
	}
}

void FMeshPaintComponentAdapterFactory::CleanupGlobals()
{
	for (const auto& Factory : FactoryList)
	{
		Factory->CleanupGlobals();
	}
}
