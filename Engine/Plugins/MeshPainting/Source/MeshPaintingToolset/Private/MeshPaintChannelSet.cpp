// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintChannelSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintChannelSet)

FMeshPaintChannelDesc const* UMeshPaintChannelSet::GetChannelDesc(FName InChannelName) const
{
	for (FMeshPaintChannelDesc const& Channel : Channels)
	{
		if (InChannelName == Channel.ChannelName)
		{
			return &Channel;
		}
	}

	return nullptr;
}
