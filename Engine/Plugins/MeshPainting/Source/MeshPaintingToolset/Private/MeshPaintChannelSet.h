// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "MeshPaintChannelSet.generated.h"

class UTexture2D;

USTRUCT(BlueprintType)
struct FMeshPaintChannelDesc
{
	GENERATED_BODY()

	/** Name of the channel that will appear in the paint tool. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	FName ChannelName;
	/** Texture that channel paints to. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	TObjectPtr<UTexture2D> PaintTexture;
	/** True if the channel writes to the red channel. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	bool bWriteRed = false;
	/** True if the channel writes to the green channel. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	bool bWriteGreen = false;
	/** True if the channel writes to the blue channel. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	bool bWriteBlue = false;
	/** True if the channel writes to the alpha channel. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	bool bWriteAlpha = false;
	/** World Units per UV unit. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	FVector2D UVToWorldScale = FVector2D::One();
	/** UV offset to apply after scaling from world into UV space. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	FVector2D WorldToUVBias = FVector2D::Zero();
	/** Minimum clamp to apply to UV after scale and bias. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	FVector2D ClampUVMin = FVector2D::Zero();
	/** Maximum clamp to apply to UV after scale and bias. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	FVector2D ClampUVMax = FVector2D::One();
};

UCLASS(BlueprintType, MinimalAPI)
class UMeshPaintChannelSet : public UDataAsset
{
	GENERATED_BODY()

public:
	/** The array of Channel Descriptions. */
	UPROPERTY(EditAnywhere, Category=MeshPaint)
	TArray<FMeshPaintChannelDesc> Channels;

	/** Return a channel with the matching ChannelName. If none is found return nullptr. */
	FMeshPaintChannelDesc const* GetChannelDesc(FName InChannelName) const;
};