// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintGeometryCollectionAdapter.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "RenderingThread.h"

DEFINE_LOG_CATEGORY_STATIC(LogMeshPaintGeometryCollectionComponentAdapter, Log, All);
//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryCollectionComponentAdapter



bool FMeshPaintGeometryCollectionComponentAdapter::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(InComponent);
	if (HasValidGeometryCollection())
	{
		GeometryCollectionChangedHandle = GeometryCollectionComponent->RegisterOnGeometryCollectionPropertyChanged(UGeometryCollectionComponent::FOnGeometryCollectionPropertyChanged::CreateSP(this, &FMeshPaintGeometryCollectionComponentAdapter::OnGeometryCollectionChanged));

		// MeshLODIndex = InMeshLODIndex; // TODO: Store the LOD index to work on, once GeometryCollection supports LODs
		const bool bSuccess = Initialize();
		return bSuccess;
	}

	return false;
}

FMeshPaintGeometryCollectionComponentAdapter::~FMeshPaintGeometryCollectionComponentAdapter()
{
	if (GeometryCollectionComponent.IsValid())
	{
		GeometryCollectionComponent->UnregisterOnGeometryCollectionPropertyChanged(GeometryCollectionChangedHandle);
	}
}

UGeometryCollection* FMeshPaintGeometryCollectionComponentAdapter::GetGeometryCollectionObject() const
{
	checkSlow(GeometryCollectionComponent.IsValid()); // expect this to be called only when the component is already verified as valid
	// Note: We bypass the FGeometryCollectionEdit machinery here because it doesn't fit with the model of the Mesh Paint Component Adapter
	// Instead, we will manually call Modify() as needed
	return const_cast<UGeometryCollection*>(GeometryCollectionComponent->GetRestCollection());
}

void FMeshPaintGeometryCollectionComponentAdapter::OnGeometryCollectionChanged()
{
	// Remove and re-initialize/add the geometry collection to make sure it is up to date after something changed
	OnRemoved();
	if (HasValidGeometryCollection())
	{
		Initialize();
		OnAdded();
	}
}

bool FMeshPaintGeometryCollectionComponentAdapter::Initialize()
{
	bool bInitializationResult = false;

	if (HasValidGeometryCollection())
	{
		bInitializationResult = FBaseMeshPaintComponentAdapter::Initialize(); // Calls InitializeVertexData() then builds octree, etc
	}
	
	return bInitializationResult;
}

bool FMeshPaintGeometryCollectionComponentAdapter::HasValidGeometryCollection() const
{
	return GeometryCollectionComponent.IsValid() && GetGeometryCollectionObject() && GetGeometryCollectionObject()->GetGeometryCollection();
}

bool FMeshPaintGeometryCollectionComponentAdapter::IsValid() const
{
	return GeometryCollectionComponent.IsValid() && GetGeometryCollectionObject() && GetGeometryCollectionObject()->GetGeometryCollection() // Pointers valid
		&& GetGeometryCollectionObject()->GetGeometryCollection()->Vertex.Num() == MeshVertices.Num(); // Data matches what we've converted
}

bool FMeshPaintGeometryCollectionComponentAdapter::InitializeVertexData()
{
	// Retrieve mesh vertex and index data 
	FGeometryCollection* Collection = GetGeometryCollectionObject()->GetGeometryCollection().Get(); 
	const TManagedArray<FVector3f>& Vertices = Collection->Vertex;
	// TODO: Revisit this when we support the same geometry mapping to multiple bones
	const TManagedArray<int32>& BoneMap = Collection->BoneMap;
	const TManagedArray<int32>& SimTypes = Collection->SimulationType;
	const TManagedArray<bool>& Visible = Collection->Visible;
	const TArray<FTransform3f>& ComponentSpaceTransforms = GeometryCollectionComponent->GetComponentSpaceTransforms3f();
	const int32 NumVertices = Vertices.Num();
	MeshVertices.Reset();
	MeshVertices.AddZeroed(NumVertices);
	for (int32 Index = 0; Index < NumVertices; ++Index)
	{
		const int32 BoneIndex = BoneMap[Index];
		const FTransform3f& Transform = ComponentSpaceTransforms[BoneIndex];
		const FVector Position = FVector(Transform.TransformPosition(Vertices[Index]));
		MeshVertices[Index] = Position;
	}

	const TManagedArray<FIntVector>& ManagedIndices = Collection->Indices;
	int32 NumTris = ManagedIndices.Num();
	for (int32 TriIdx = 0; TriIdx < NumTris; ++TriIdx)
	{
		if (Visible[TriIdx])
		{
			MeshIndices.Add(ManagedIndices[TriIdx].X);
			MeshIndices.Add(ManagedIndices[TriIdx].Y);
			MeshIndices.Add(ManagedIndices[TriIdx].Z);
		}
	}

	return (MeshVertices.Num() >= 0 && MeshIndices.Num() > 0);
}

void FMeshPaintGeometryCollectionComponentAdapter::InitializeAdapterGlobals()
{
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
	}
}

void FMeshPaintGeometryCollectionComponentAdapter::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
}

void FMeshPaintGeometryCollectionComponentAdapter::AddReferencedObjects(FReferenceCollector& Collector)
{
}

void FMeshPaintGeometryCollectionComponentAdapter::CleanupGlobals()
{
}

void FMeshPaintGeometryCollectionComponentAdapter::OnAdded()
{
	// We shouldn't assume that the cached geometry collection component remains valid.
	// Components may be destroyed by editor ticks, and be forcibly removed by GC.
	if (!GeometryCollectionComponent.IsValid())
	{
		return;
	}
	bSavedShowBoneColors = GeometryCollectionComponent->GetShowBoneColors();
	GeometryCollectionComponent->SetShowBoneColors(false);
	GeometryCollectionComponent->ForceNativeRendering(true);

	checkf(GetGeometryCollectionObject(), TEXT("Geometry Collection Component did not have a valid geometry collection object attached"));
	checkf(GetGeometryCollectionObject()->GetGeometryCollection(), TEXT("Geometry Collection Component did not have valid geometry collection data attached"));
}

void FMeshPaintGeometryCollectionComponentAdapter::OnRemoved()
{
	if (GeometryCollectionComponent.IsValid())
	{
		GeometryCollectionComponent->SetShowBoneColors(bSavedShowBoneColors);
		bSavedShowBoneColors = false;
		GeometryCollectionComponent->ForceNativeRendering(false);
	}
}

int32 FMeshPaintGeometryCollectionComponentAdapter::GetNumUVChannels() const
{
	if (!IsValid())
	{
		return 0;
	}
	return GetGeometryCollectionObject()->GetGeometryCollection()->NumUVLayers();
}

bool FMeshPaintGeometryCollectionComponentAdapter::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	// Note: This is 99% the same as the skeletal mesh component adapter version, and could likely be pushed up to the base class if we added a generalized way to get the Component

	if (!IsValid())
	{
		return false;
	}

	const bool bHitBounds = FMath::LineSphereIntersection(Start, End.GetSafeNormal(), (End - Start).SizeSquared(), GeometryCollectionComponent->Bounds.Origin, GeometryCollectionComponent->Bounds.SphereRadius);
	const float SqrRadius = FMath::Square(GeometryCollectionComponent->Bounds.SphereRadius);
	const bool bInsideBounds = (GeometryCollectionComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(Start) <= SqrRadius) || (GeometryCollectionComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(End) <= SqrRadius);

	bool bHitTriangle = false;
	if (bHitBounds || bInsideBounds)
	{
		const FTransform& ComponentTransform = GeometryCollectionComponent->GetComponentTransform();
		const FVector LocalStart = ComponentTransform.InverseTransformPosition(Start);
		const FVector LocalEnd = ComponentTransform.InverseTransformPosition(End);
		float MinDistance = FLT_MAX;
		FVector Intersect;
		FVector Normal;
		UE::Geometry::FIndex3i FoundTriangle;
		FVector HitPosition;
		if (!RayIntersectAdapter(FoundTriangle, HitPosition, LocalStart, LocalEnd))
		{
			return false;
		}

		// Compute the normal of the triangle
		const FVector& P0 = MeshVertices[FoundTriangle.A];
		const FVector& P1 = MeshVertices[FoundTriangle.B];
		const FVector& P2 = MeshVertices[FoundTriangle.C];

		const FVector TriNorm = (P1 - P0) ^ (P2 - P0);

		//check collinearity of A,B,C
		if (TriNorm.SizeSquared() > SMALL_NUMBER)
		{
			FVector IntersectPoint;
			FVector HitNormal;
		
			bool bHit = FMath::SegmentTriangleIntersection(LocalStart, LocalEnd, P0, P1, P2, IntersectPoint, HitNormal);

			if (bHit)
			{
				const float Distance = (LocalStart - IntersectPoint).SizeSquared();
				if (Distance < MinDistance)
				{
					MinDistance = Distance;
					Intersect = IntersectPoint;
					Normal = HitNormal;
				}
			}
		}
		

		if (MinDistance != FLT_MAX)
		{
			OutHit.Component = GeometryCollectionComponent;
			OutHit.Normal = ComponentTransform.TransformVector(Normal).GetSafeNormal();
			OutHit.ImpactNormal = OutHit.Normal;
			OutHit.ImpactPoint = ComponentTransform.TransformPosition(Intersect);
			OutHit.Location = OutHit.ImpactPoint;
			OutHit.bBlockingHit = true;
			OutHit.Distance = MinDistance;
			bHitTriangle = true;
		}
	}	

	return bHitTriangle;
}

void FMeshPaintGeometryCollectionComponentAdapter::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	if (GeometryCollectionComponent.IsValid())
	{
		DefaultQueryPaintableTextures(MaterialIndex, GeometryCollectionComponent.Get(), OutDefaultIndex, InOutTextureList);
	}
}

void FMeshPaintGeometryCollectionComponentAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const
{
	if (GeometryCollectionComponent.IsValid())
	{
		TextureOverridesState.ApplyOrRemoveTextureOverride(GeometryCollectionComponent.Get(), SourceTexture, OverrideTexture, MaterialUpdateContext);
	}
}

void FMeshPaintGeometryCollectionComponentAdapter::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	const TManagedArray<FVector2f>* UVLayer = GetGeometryCollectionObject()->GetGeometryCollection()->FindUVLayer(ChannelIndex);
	if (UVLayer)
	{
		OutTextureCoordinate = (FVector2D)(*UVLayer)[VertexIndex];
	}
}

void FMeshPaintGeometryCollectionComponentAdapter::PreEdit()
{
	if (!IsValid())
	{
		return;
	}

	FlushRenderingCommands();

	GeometryCollectionComponent->Modify();
	GetGeometryCollectionObject()->Modify();
}

void FMeshPaintGeometryCollectionComponentAdapter::PostEdit()
{
	if (!GeometryCollectionComponent.IsValid())
	{
		return;
	}

	GetGeometryCollectionObject()->RebuildRenderData();
}

void FMeshPaintGeometryCollectionComponentAdapter::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	OutColor = GetGeometryCollectionObject()->GetGeometryCollection()->Color[VertexIndex].ToFColor(true);
}

void FMeshPaintGeometryCollectionComponentAdapter::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	GetGeometryCollectionObject()->GetGeometryCollection()->Color[VertexIndex] = FLinearColor(Color);
}

FMatrix FMeshPaintGeometryCollectionComponentAdapter::GetComponentToWorldMatrix() const
{
	if (!GeometryCollectionComponent.IsValid())
	{
		return FMatrix::Identity;
	}

	return GeometryCollectionComponent->GetComponentToWorld().ToMatrixWithScale();
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryCollectionComponentAdapterFactory

TSharedPtr<IMeshPaintComponentAdapter> FMeshPaintGeometryCollectionComponentAdapterFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(InComponent))
	{
		if (GeometryCollectionComponent->GetRestCollection() != nullptr)
		{
			TSharedRef<FMeshPaintGeometryCollectionComponentAdapter> Result = MakeShareable(new FMeshPaintGeometryCollectionComponentAdapter());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
