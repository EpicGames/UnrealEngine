// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintHelpers.h"

#include "ComponentReregisterContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/Texture2D.h"
#include "IMeshPaintComponentAdapter.h"
#include "InteractiveTool.h"
#include "MeshPaintAdapterFactory.h"
#include "MeshVertexPaintingTool.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshComponentLODInfo.h"
#include "TexturePaintToolset.h"
#include "VT/MeshPaintVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintHelpers)

extern void PropagateVertexPaintToSkeletalMesh(USkeletalMesh* SkeletalMesh, int32 LODIndex);

UMeshPaintingSubsystem::UMeshPaintingSubsystem()
	: bNeedsRecache(true)
	, bSelectionSupportsVertexPaint(false)
	, bSelectionSupportsTextureColorPaint(false)
	, bSelectionSupportsTextureAssetPaint(false)
{

}

bool UMeshPaintingSubsystem::HasPaintableMesh(UActorComponent* Component)
{
	return Cast<UMeshComponent>(Component) != nullptr;
}

void UMeshPaintingSubsystem::RemoveInstanceVertexColors(UObject* Obj)
{
	// Currently only static mesh component supports per instance vertex colors so only need to retrieve those and remove colors
	AActor* Actor = Cast<AActor>(Obj);
	if (Actor != nullptr)
	{
		TArray<UStaticMeshComponent*> StaticMeshComponents;
		Actor->GetComponents(StaticMeshComponents);
		for (const auto& StaticMeshComponent : StaticMeshComponents)
		{
			if (StaticMeshComponent != nullptr)
			{
				UMeshPaintingSubsystem::RemoveComponentInstanceVertexColors(StaticMeshComponent);
			}
		}
	}
}

void UMeshPaintingSubsystem::RemoveComponentInstanceVertexColors(UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr && StaticMeshComponent->GetStaticMesh() != nullptr)
	{
		// Mark the mesh component as modified
		StaticMeshComponent->Modify();

		// If this is called from the Remove button being clicked the SMC wont be in a Reregister context,
		// but when it gets called from a Paste or Copy to Source operation it's already inside a more specific
		// SMCRecreateScene context so we shouldn't put it inside another one.
		if (StaticMeshComponent->IsRenderStateCreated())
		{
			// Detach all instances of this static mesh from the scene.
			FComponentReregisterContext ComponentReregisterContext(StaticMeshComponent);

			StaticMeshComponent->RemoveInstanceVertexColors();
		}
		else
		{
			StaticMeshComponent->RemoveInstanceVertexColors();
		}
	}
}

UTexture* UMeshPaintingSubsystem::CreateMeshPaintTexture(UObject* Outer,  uint32 TextureSize)
{
	const uint32 AlignedTextureSize = MeshPaintVirtualTexture::GetAlignedTextureSize(TextureSize);
	const uint32 TextureNumMips = FMath::FloorLog2(AlignedTextureSize) + 1;

	UMeshPaintVirtualTexture* NewTexture = NewObject<UMeshPaintVirtualTexture>(Outer);
	NewTexture->Source.Init(AlignedTextureSize, AlignedTextureSize, 1, TextureNumMips, TSF_BGRA8);
	NewTexture->UpdateResource();

	return NewTexture;
}

void UMeshPaintingSubsystem::CreateComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr && StaticMeshComponent->GetMeshPaintTexture() == nullptr && StaticMeshComponent->CanMeshPaintTextureColors())
	{
		StaticMeshComponent->Modify();

		const uint32 TextureSize = StaticMeshComponent->GetMeshPaintTextureResolution();
		const uint32 TextureNumMips = FMath::FloorLog2(TextureSize) + 1;

		UMeshPaintVirtualTexture* NewTexture = NewObject<UMeshPaintVirtualTexture>(StaticMeshComponent);
		NewTexture->Source.Init(TextureSize, TextureSize, 1, TextureNumMips, TSF_BGRA8);
		NewTexture->OwningComponent = MakeWeakObjectPtr(StaticMeshComponent);
		NewTexture->UpdateResource();

		StaticMeshComponent->SetMeshPaintTexture(NewTexture);
	}
}

void UMeshPaintingSubsystem::CreateComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent, FImageView const& InImage)
{
	if (StaticMeshComponent != nullptr)
	{
		StaticMeshComponent->Modify();

		UMeshPaintVirtualTexture* NewTexture = NewObject<UMeshPaintVirtualTexture>(StaticMeshComponent);
		NewTexture->Source.Init(InImage);
		NewTexture->OwningComponent = MakeWeakObjectPtr(StaticMeshComponent);
		NewTexture->UpdateResource();

		StaticMeshComponent->SetMeshPaintTexture(NewTexture);
	}
}

void UMeshPaintingSubsystem::RemoveComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr && StaticMeshComponent->GetMeshPaintTexture() != nullptr)
	{
		// Mark the mesh component as modified
		StaticMeshComponent->Modify();

		StaticMeshComponent->SetMeshPaintTexture(nullptr);
	}
}

bool UMeshPaintingSubsystem::PropagateColorsToRawMesh(UStaticMesh* StaticMesh, int32 LODIndex, FStaticMeshComponentLODInfo& ComponentLODInfo)
{
	if (!ComponentLODInfo.OverrideVertexColors ||
		!StaticMesh ||
		!StaticMesh->IsSourceModelValid(LODIndex) ||
		!StaticMesh->GetRenderData() ||
		!StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex) ||
		StaticMesh->GetOutermost()->bIsCookedForEditor)
	{
		return false;
	}

	bool bPropagatedColors = false;
	FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
	FStaticMeshRenderData& RenderData = *StaticMesh->GetRenderData();
	FStaticMeshLODResources& RenderModel = RenderData.LODResources[LODIndex];
	FColorVertexBuffer& ColorVertexBuffer = *ComponentLODInfo.OverrideVertexColors;

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

	if (MeshDescription == nullptr)
	{
		return false;
	}

	// Use the wedge map if it is available as it is lossless.
	if (RenderModel.WedgeMap.Num() > 0 && ColorVertexBuffer.GetNumVertices() == RenderModel.GetNumVertices() && RenderModel.WedgeMap.Num() == MeshDescription->VertexInstances().Num())
	{
		FStaticMeshAttributes Attributes(*MeshDescription);
		TVertexInstanceAttributesRef<FVector4f> Colors = Attributes.GetVertexInstanceColors();

		int32 VertexInstanceIndex = 0;
		for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
		{
			FLinearColor WedgeColor = FLinearColor::White;
			int32 Index = RenderModel.WedgeMap[VertexInstanceIndex];
			if (Index != INDEX_NONE)
			{
				WedgeColor = FLinearColor(ColorVertexBuffer.VertexColor(Index));
			}
			Colors[VertexInstanceID] = WedgeColor;
			VertexInstanceIndex++;
		}
		StaticMesh->CommitMeshDescription(LODIndex);
		bPropagatedColors = true;
	}
	else // Fall back to mapping based on position.
	{
		// If there's no raw mesh data, don't try to do any fixup here
		if (ComponentLODInfo.OverrideVertexColors == nullptr)
		{
			return false;
		}

		FStaticMeshAttributes Attributes(*MeshDescription);

		TVertexInstanceAttributesRef<FVector4f> Colors = Attributes.GetVertexInstanceColors();
		TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TArray<FColor> NewVertexColors;
		FPositionVertexBuffer TempPositionVertexBuffer;
		int32 NumVertex = MeshDescription->Vertices().Num();
		TArray<FVector3f> VertexPositionsDup;
		VertexPositionsDup.AddZeroed(NumVertex);
		int32 VertexIndex = 0;
		for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
		{
			VertexPositionsDup[VertexIndex++] = VertexPositions[VertexID];
		}
		TempPositionVertexBuffer.Init(VertexPositionsDup);
		RemapPaintedVertexColors(
			ComponentLODInfo.PaintedVertices,
			ComponentLODInfo.OverrideVertexColors,
			RenderModel.VertexBuffers.PositionVertexBuffer,
			RenderModel.VertexBuffers.StaticMeshVertexBuffer,
			TempPositionVertexBuffer,
			/*OptionalVertexBuffer=*/ nullptr,
			NewVertexColors
		);
		if (NewVertexColors.Num() == NumVertex)
		{
			for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
			{
				const FVertexID VertexID = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
				Colors[VertexInstanceID] = FLinearColor(NewVertexColors[VertexID.GetValue()]);
			}
			StaticMesh->CommitMeshDescription(LODIndex);
			bPropagatedColors = true;
		}
	}
	return bPropagatedColors;
}

bool UMeshPaintingSubsystem::PaintVertex(const FVector& InVertexPosition, const FMeshPaintParameters& InParams, FColor& InOutVertexColor)
{
	float SquaredDistanceToVertex2D;
	float VertexDepthToBrush;
	if (UMeshPaintingSubsystem::IsPointInfluencedByBrush(InVertexPosition, InParams, SquaredDistanceToVertex2D, VertexDepthToBrush))
	{
		// Compute amount of paint to apply
		const float PaintAmount = UMeshPaintingSubsystem::ComputePaintMultiplier(SquaredDistanceToVertex2D, InParams.BrushStrength, InParams.InnerBrushRadius, InParams.BrushRadialFalloffRange, InParams.BrushDepth, InParams.BrushDepthFalloffRange, VertexDepthToBrush);
			
		const FLinearColor OldColor = InOutVertexColor.ReinterpretAsLinear();
		FLinearColor NewColor = OldColor;

		InParams.ApplyVertexDataDelegate.Broadcast(InParams, OldColor, NewColor, PaintAmount);

		// Save the new color
		InOutVertexColor.R = FMath::Clamp(FMath::RoundToInt(NewColor.R * 255.0f), 0, 255);
		InOutVertexColor.G = FMath::Clamp(FMath::RoundToInt(NewColor.G * 255.0f), 0, 255);
		InOutVertexColor.B = FMath::Clamp(FMath::RoundToInt(NewColor.B * 255.0f), 0, 255);
		InOutVertexColor.A = FMath::Clamp(FMath::RoundToInt(NewColor.A * 255.0f), 0, 255);

		return true;
	}

	// Out of range
	return false;
}

void UMeshPaintingSubsystem::ApplyVertexColorPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount)
{
	// Color painting

	if (InParams.bWriteRed)
	{
		NewColor.R = (OldColor.R < InParams.BrushColor.R) ? FMath::Min(InParams.BrushColor.R, OldColor.R + PaintAmount) : FMath::Max(InParams.BrushColor.R, OldColor.R - PaintAmount);
	}

	if (InParams.bWriteGreen)
	{
		NewColor.G = (OldColor.G < InParams.BrushColor.G) ? FMath::Min(InParams.BrushColor.G, OldColor.G + PaintAmount) : FMath::Max(InParams.BrushColor.G, OldColor.G - PaintAmount);
	}

	if (InParams.bWriteBlue)
	{
		NewColor.B = (OldColor.B < InParams.BrushColor.B) ? FMath::Min(InParams.BrushColor.B, OldColor.B + PaintAmount) : FMath::Max(InParams.BrushColor.B, OldColor.B - PaintAmount);
	}

	if (InParams.bWriteAlpha)
	{
		NewColor.A = (OldColor.A < InParams.BrushColor.A) ? FMath::Min(InParams.BrushColor.A, OldColor.A + PaintAmount) : FMath::Max(InParams.BrushColor.A, OldColor.A - PaintAmount);
	}
}

void UMeshPaintingSubsystem::ApplyVertexWeightPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount)
{
	// Total number of texture blend weights we're using
	check(InParams.TotalWeightCount > 1);
	check(InParams.TotalWeightCount <= MeshPaintDefs::MaxSupportedWeights);

	// True if we should assume the last weight index is composed of one minus the sum of all
	// of the other weights.  This effectively allows an additional weight with no extra memory
	// used, but potentially requires extra pixel shader instructions to render.
	//
	// NOTE: If you change the default here, remember to update the MeshPaintWindow UI and strings
	//
	// NOTE: Materials must be authored to match the following assumptions!
	const bool bUsingOneMinusTotal =
		InParams.TotalWeightCount == 2 ||		// Two textures: Use a lerp() in pixel shader (single value)
		InParams.TotalWeightCount == 5;			// Five texture: Requires 1.0-sum( R+G+B+A ) in shader
	check(bUsingOneMinusTotal || InParams.TotalWeightCount <= MeshPaintDefs::MaxSupportedPhysicalWeights);

	// Prefer to use RG/RGB instead of AR/ARG when we're only using 2/3 physical weights
	const int32 TotalPhysicalWeights = bUsingOneMinusTotal ? InParams.TotalWeightCount - 1 : InParams.TotalWeightCount;
	const bool bUseColorAlpha =
		TotalPhysicalWeights != 2 &&			// Two physical weights: Use RG instead of AR
		TotalPhysicalWeights != 3;				// Three physical weights: Use RGB instead of ARG

	// Index of the blend weight that we're painting
	check(InParams.PaintWeightIndex >= 0 && InParams.PaintWeightIndex < MeshPaintDefs::MaxSupportedWeights);

	// Convert the color value to an array of weights
	float Weights[MeshPaintDefs::MaxSupportedWeights];
	{
		for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
		{
			if (CurWeightIndex == TotalPhysicalWeights)
			{
				// This weight's value is one minus the sum of all previous weights
				float OtherWeightsTotal = 0.0f;
				for (int32 OtherWeightIndex = 0; OtherWeightIndex < CurWeightIndex; ++OtherWeightIndex)
				{
					OtherWeightsTotal += Weights[OtherWeightIndex];
				}
				Weights[CurWeightIndex] = 1.0f - OtherWeightsTotal;
			}
			else
			{
				switch (CurWeightIndex)
				{
				case 0:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.A : OldColor.R;
					break;

				case 1:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.R : OldColor.G;
					break;

				case 2:
					Weights[CurWeightIndex] = bUseColorAlpha ? OldColor.G : OldColor.B;
					break;

				case 3:
					check(bUseColorAlpha);
					Weights[CurWeightIndex] = OldColor.B;
					break;
				}
			}
		}
	}

	// Go ahead any apply paint!	
	Weights[InParams.PaintWeightIndex] += PaintAmount;
	Weights[InParams.PaintWeightIndex] = FMath::Clamp(Weights[InParams.PaintWeightIndex], 0.0f, 1.0f);
	

	// Now renormalize all of the other weights	
	float OtherWeightsTotal = 0.0f;
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		if (CurWeightIndex != InParams.PaintWeightIndex)
		{
			OtherWeightsTotal += Weights[CurWeightIndex];
		}
	}
	const float NormalizeTarget = 1.0f - Weights[InParams.PaintWeightIndex];
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		if (CurWeightIndex != InParams.PaintWeightIndex)
		{
			if (OtherWeightsTotal == 0.0f)
			{
				Weights[CurWeightIndex] = NormalizeTarget / (InParams.TotalWeightCount - 1);
			}
			else
			{
				Weights[CurWeightIndex] = Weights[CurWeightIndex] / OtherWeightsTotal * NormalizeTarget;
			}
		}
	}

	// The total of the weights should now always equal 1.0	
	float WeightsTotal = 0.0f;
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		WeightsTotal += Weights[CurWeightIndex];
	}
	check(FMath::IsNearlyEqual(WeightsTotal, 1.0f, 0.01f));
	
	// Convert the weights back to a color value	
	for (int32 CurWeightIndex = 0; CurWeightIndex < InParams.TotalWeightCount; ++CurWeightIndex)
	{
		// We can skip the non-physical weights as it's already baked into the others
		if (CurWeightIndex != TotalPhysicalWeights)
		{
			switch (CurWeightIndex)
			{
			case 0:
				if (bUseColorAlpha)
				{
					NewColor.A = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				break;

			case 1:
				if (bUseColorAlpha)
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				break;

			case 2:
				if (bUseColorAlpha)
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.B = Weights[CurWeightIndex];
				}
				break;

			case 3:
				NewColor.B = Weights[CurWeightIndex];
				break;
			}
		}
	}	
}

FLinearColor UMeshPaintingSubsystem::GenerateColorForTextureWeight(const int32 NumWeights, const int32 WeightIndex)
{
	const bool bUsingOneMinusTotal =
		NumWeights == 2 ||		// Two textures: Use a lerp() in pixel shader (single value)
		NumWeights == 5;			// Five texture: Requires 1.0-sum( R+G+B+A ) in shader
	check(bUsingOneMinusTotal || NumWeights <= MeshPaintDefs::MaxSupportedPhysicalWeights);

	// Prefer to use RG/RGB instead of AR/ARG when we're only using 2/3 physical weights
	const int32 TotalPhysicalWeights = bUsingOneMinusTotal ? NumWeights - 1 : NumWeights;
	const bool bUseColorAlpha =
		TotalPhysicalWeights != 2 &&			// Two physical weights: Use RG instead of AR
		TotalPhysicalWeights != 3;				// Three physical weights: Use RGB instead of ARG

												// Index of the blend weight that we're painting
	check(WeightIndex >= 0 && WeightIndex < MeshPaintDefs::MaxSupportedWeights);

	// Convert the color value to an array of weights
	float Weights[MeshPaintDefs::MaxSupportedWeights];
	{
		for (int32 CurWeightIndex = 0; CurWeightIndex < NumWeights; ++CurWeightIndex)
		{
			if (CurWeightIndex == TotalPhysicalWeights)
			{
				// This weight's value is one minus the sum of all previous weights
				float OtherWeightsTotal = 0.0f;
				for (int32 OtherWeightIndex = 0; OtherWeightIndex < CurWeightIndex; ++OtherWeightIndex)
				{
					OtherWeightsTotal += Weights[OtherWeightIndex];
				}
				Weights[CurWeightIndex] = 1.0f - OtherWeightsTotal;
			}
			else
			{
				if (CurWeightIndex == WeightIndex)
				{
					Weights[CurWeightIndex] = 1.0f;
				}
				else
				{
					Weights[CurWeightIndex] = 0.0f;
				}
			}
		}
	}

	FLinearColor NewColor(FLinearColor::Black);
	// Convert the weights back to a color value	
	for (int32 CurWeightIndex = 0; CurWeightIndex < NumWeights; ++CurWeightIndex)
	{
		// We can skip the non-physical weights as it's already baked into the others
		if (CurWeightIndex != TotalPhysicalWeights)
		{
			switch (CurWeightIndex)
			{
			case 0:
				if (bUseColorAlpha)
				{
					NewColor.A = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				break;

			case 1:
				if (bUseColorAlpha)
				{
					NewColor.R = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				break;

			case 2:
				if (bUseColorAlpha)
				{
					NewColor.G = Weights[CurWeightIndex];
				}
				else
				{
					NewColor.B = Weights[CurWeightIndex];
				}
				break;

			case 3:
				NewColor.B = Weights[CurWeightIndex];
				break;
			}
		}
	}

	return NewColor;
}

float UMeshPaintingSubsystem::ComputePaintMultiplier(float SquaredDistanceToVertex2D, float BrushStrength, float BrushInnerRadius, float BrushRadialFalloff, float BrushInnerDepth, float BrushDepthFallof, float VertexDepthToBrush)
{
	float PaintAmount = 1.0f;
	
	// Compute the actual distance
	float DistanceToVertex2D = 0.0f;
	if (SquaredDistanceToVertex2D > KINDA_SMALL_NUMBER)
	{
		DistanceToVertex2D = FMath::Sqrt(SquaredDistanceToVertex2D);
	}

	// Apply radial-based falloff
	if (DistanceToVertex2D > BrushInnerRadius)
	{
		const float RadialBasedFalloff = (DistanceToVertex2D - BrushInnerRadius) / BrushRadialFalloff;
		PaintAmount *= 1.0f - RadialBasedFalloff;
	}	

	// Apply depth-based falloff	
	if (VertexDepthToBrush > BrushInnerDepth)
	{
		const float DepthBasedFalloff = (VertexDepthToBrush - BrushInnerDepth) / BrushDepthFallof;
		PaintAmount *= 1.0f - DepthBasedFalloff;
	}

	PaintAmount *= BrushStrength;

	return PaintAmount;
}

bool UMeshPaintingSubsystem::IsPointInfluencedByBrush(const FVector& InPosition, const FMeshPaintParameters& InParams, float& OutSquaredDistanceToVertex2D, float& OutVertexDepthToBrush)
{
	// Project the vertex into the plane of the brush
	FVector BrushSpaceVertexPosition = InParams.InverseBrushToWorldMatrix.TransformPosition(InPosition);
	FVector2D BrushSpaceVertexPosition2D(BrushSpaceVertexPosition.X, BrushSpaceVertexPosition.Y);

	// Is the brush close enough to the vertex to paint?
	const float SquaredDistanceToVertex2D = BrushSpaceVertexPosition2D.SizeSquared();
	if (SquaredDistanceToVertex2D <= InParams.SquaredBrushRadius)
	{
		// OK the vertex is overlapping the brush in 2D space, but is it too close or
		// two far (depth wise) to be influenced?
		const float VertexDepthToBrush = FMath::Abs(BrushSpaceVertexPosition.Z);
		if (VertexDepthToBrush <= InParams.BrushDepth)
		{
			OutSquaredDistanceToVertex2D = SquaredDistanceToVertex2D;
			OutVertexDepthToBrush = VertexDepthToBrush;
			return true;
		}
	}

	return false;
}

bool UMeshPaintingSubsystem::IsPointInfluencedByBrush(const FVector2D& BrushSpacePosition, const float BrushRadius, float& OutInRangeValue)
{
	const float DistanceToBrush = BrushSpacePosition.SizeSquared();
	if (DistanceToBrush <= BrushRadius)
	{
		OutInRangeValue = DistanceToBrush / BrushRadius;
		return true;
	}

	return false;
}

uint32 UMeshPaintingSubsystem::GetVertexColorBufferSize(UMeshComponent* MeshComponent, int32 LODIndex, bool bInstance)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));
	
	uint32 SizeInBytes = 0;

	// Retrieve component instance vertex color buffer size
	if (bInstance)
	{
		if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			if (StaticMeshComponent->LODData.IsValidIndex(LODIndex))
			{
				const FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[LODIndex];
				if (InstanceMeshLODInfo.OverrideVertexColors)
				{
					SizeInBytes = InstanceMeshLODInfo.OverrideVertexColors->GetAllocatedSize();
				}
			}
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			if (SkinnedMeshComponent->LODInfo.IsValidIndex(LODIndex))
			{
				const FSkelMeshComponentLODInfo& LODInfo = SkinnedMeshComponent->LODInfo[LODIndex];
				if (LODInfo.OverrideVertexColors)
				{
					SizeInBytes = LODInfo.OverrideVertexColors->GetAllocatedSize();
				}
			}
		}
	}
	// Retrieve static mesh asset vertex color buffer size
	else
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));
			if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
			{
				// count the base mesh color data
				FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
				SizeInBytes = LODModel.VertexBuffers.ColorVertexBuffer.GetAllocatedSize();
			}
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			FSkeletalMeshRenderData* RenderData = SkinnedMeshComponent->GetSkeletalMeshRenderData();
			if (RenderData && RenderData->LODRenderData.IsValidIndex(LODIndex))
			{
				SizeInBytes = RenderData->LODRenderData[LODIndex].StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize();
			}
		}
	}
	
	return SizeInBytes;
}

uint32 UMeshPaintingSubsystem::GetMeshPaintTextureResourceSize(UMeshComponent* MeshComponent)
{
	if (UTexture* Texture = MeshComponent->GetMeshPaintTexture())
	{
		// Check that the texture has finished compilation before reading platform data.
		if (!Texture->IsDefaultTexture())
		{
			FTexturePlatformData** PlatformDataPtr = Texture->GetRunningPlatformData();
			if (PlatformDataPtr != nullptr && *PlatformDataPtr != nullptr)
			{
				return (*PlatformDataPtr)->GetPayloadSize(0);
			}
		}
	}
	return 0;
}

TArray<FVector> UMeshPaintingSubsystem::GetVerticesForLOD( const UStaticMesh* StaticMesh, int32 LODIndex)
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));

	// Retrieve mesh vertices from Static mesh render data 
	TArray<FVector> Vertices;
	if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
		const FPositionVertexBuffer* VertexBuffer = &LODModel.VertexBuffers.PositionVertexBuffer;
		const uint32 NumVertices = VertexBuffer->GetNumVertices();
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			Vertices.Add((FVector)VertexBuffer->VertexPosition(VertexIndex));
		}		
	}
	return Vertices;
}

TArray<FColor> UMeshPaintingSubsystem::GetColorDataForLOD( const UStaticMesh* StaticMesh, int32 LODIndex)
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh ptr"));
	// Retrieve mesh vertex colors from Static mesh render data 
	TArray<FColor> Colors;
	if (StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
	{
		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
		const FColorVertexBuffer& ColorBuffer = LODModel.VertexBuffers.ColorVertexBuffer;
		const uint32 NumColors = ColorBuffer.GetNumVertices();
		for (uint32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
		{
			Colors.Add(ColorBuffer.VertexColor(ColorIndex));
		}
	}
	return Colors;
}

TArray<FColor> UMeshPaintingSubsystem::GetInstanceColorDataForLOD(const UStaticMeshComponent* MeshComponent, int32 LODIndex)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));
	TArray<FColor> Colors;
	
	// Retrieve mesh vertex colors from Static Mesh component instance data
	if (MeshComponent->LODData.IsValidIndex(LODIndex))
	{
		const FStaticMeshComponentLODInfo& ComponentLODInfo = MeshComponent->LODData[LODIndex];
		const FColorVertexBuffer* ColorBuffer = ComponentLODInfo.OverrideVertexColors;
		if (ColorBuffer)
		{
			const uint32 NumColors = ColorBuffer->GetNumVertices();
			for (uint32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
			{
				Colors.Add(ColorBuffer->VertexColor(ColorIndex));
			}
		}
	}

	return Colors;
}

void UMeshPaintingSubsystem::SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const TArray<FColor>& Colors)
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));

	const UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const FStaticMeshLODResources& RenderData = Mesh->GetRenderData()->LODResources[LODIndex];
		FStaticMeshComponentLODInfo& ComponentLodInfo = MeshComponent->LODData[LODIndex];		

		// First release existing buffer
		if (ComponentLodInfo.OverrideVertexColors)
		{
			ComponentLodInfo.ReleaseOverrideVertexColorsAndBlock();
		}

		// If we are adding colors to LOD > 0 we flag the component to have per-lod painted mesh colors
		if (LODIndex > 0)
		{
			MeshComponent->bCustomOverrideVertexColorPerLOD = true;			
		}

		// Initialize vertex buffer from given colors
		ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
		ComponentLodInfo.OverrideVertexColors->InitFromColorArray(Colors);
		
		//Update the cache painted vertices
		ComponentLodInfo.PaintedVertices.Empty();
		MeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(ComponentLodInfo.OverrideVertexColors);
	}
}

void UMeshPaintingSubsystem::SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor )
{
	checkf(MeshComponent != nullptr, TEXT("Invalid static mesh component ptr"));

	const UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const FStaticMeshLODResources& RenderData = Mesh->GetRenderData()->LODResources[LODIndex];
		// Ensure we have enough LOD data structs
		MeshComponent->SetLODDataCount(LODIndex + 1, MeshComponent->LODData.Num());
		FStaticMeshComponentLODInfo& ComponentLodInfo = MeshComponent->LODData[LODIndex];

		if (MaskColor == FColor::White)
		{
			// First release existing buffer
			if (ComponentLodInfo.OverrideVertexColors)
			{
				ComponentLodInfo.ReleaseOverrideVertexColorsAndBlock();
			}

			// If we are adding colors to LOD > 0 we flag the component to have per-lod painted mesh colors
			if (LODIndex > 0)
			{
				MeshComponent->bCustomOverrideVertexColorPerLOD = true;
			}

			// Initialize vertex buffer from given color
			ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
			ComponentLodInfo.OverrideVertexColors->InitFromSingleColor(FillColor, RenderData.GetNumVertices());			
		}
		else
		{
			const FStaticMeshLODResources& LODModel = MeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];
			/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
			if (ComponentLodInfo.OverrideVertexColors)
			{
				const uint32 NumVertices = ComponentLodInfo.OverrideVertexColors->GetNumVertices();
				for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{					
					ApplyFillWithMask(ComponentLodInfo.OverrideVertexColors->VertexColor(VertexIndex), MaskColor, FillColor);
				}
			}
			else
			{
				// Initialize vertex buffer from given color
				ComponentLodInfo.OverrideVertexColors = new FColorVertexBuffer;
				FColor NewFillColor(EForceInit::ForceInitToZero);
				ApplyFillWithMask(NewFillColor, MaskColor, FillColor);
				ComponentLodInfo.OverrideVertexColors->InitFromSingleColor(NewFillColor, RenderData.GetNumVertices());
			}
		}

		//Update the cache painted vertices
		ComponentLodInfo.PaintedVertices.Empty();
		MeshComponent->CachePaintedDataIfNecessary();

		BeginInitResource(ComponentLodInfo.OverrideVertexColors);
	}
}

void UMeshPaintingSubsystem::FillStaticMeshVertexColors(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor)
{
	UStaticMesh* Mesh = MeshComponent->GetStaticMesh();
	if (Mesh)
	{
		const int32 NumLODs = Mesh->GetNumLODs();
		if (LODIndex < NumLODs)
		{
			if (LODIndex == -1)
			{
				for (LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
				{
					UMeshPaintingSubsystem::SetInstanceColorDataForLOD(MeshComponent, LODIndex, FillColor, MaskColor);
				}
			}
			else
			{
				UMeshPaintingSubsystem::SetInstanceColorDataForLOD(MeshComponent, LODIndex, FillColor, MaskColor);
			}
		}
	}
}

void UMeshPaintingSubsystem::FillSkeletalMeshVertexColors(USkeletalMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor)
{
	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext;
	USkeletalMesh* Mesh = MeshComponent->GetSkeletalMeshAsset();
	if (Mesh)
	{
		// Dirty the mesh
		Mesh->SetFlags(RF_Transactional);
		Mesh->Modify();
		Mesh->SetHasVertexColors(true);
		Mesh->SetVertexColorGuid(FGuid::NewGuid());

		// Release the static mesh's resources.
		Mesh->ReleaseResources();

		// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
		// allocated, and potentially accessing the UStaticMesh.
		Mesh->ReleaseResourcesFence.Wait();

		const int32 NumLODs = Mesh->GetLODNum();
		if (NumLODs > 0)
		{
			RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(Mesh);
			// TODO: Apply to LODIndex only (or all if set to -1). This requires some extra refactoring
			// because currently all LOD data is being released above.
			for (LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				UMeshPaintingSubsystem::SetColorDataForLOD(Mesh, LODIndex, FillColor, MaskColor);
				PropagateVertexPaintToSkeletalMesh(Mesh, LODIndex);
			}
			Mesh->InitResources();
		}
	}
}

void UMeshPaintingSubsystem::SetColorDataForLOD(USkeletalMesh* SkeletalMesh, int32 LODIndex, const FColor FillColor, const FColor MaskColor )
{
	checkf(SkeletalMesh != nullptr, TEXT("Invalid Skeletal Mesh Ptr"));
	FSkeletalMeshRenderData* Resource = SkeletalMesh->GetResourceForRendering();
	if (Resource && Resource->LODRenderData.IsValidIndex(LODIndex))
	{
		FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[LODIndex];

		if (MaskColor == FColor::White)
		{
			LODData.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FillColor, LODData.GetNumVertices());
		}
		else
		{
			/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
			const uint32 NumVertices = LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices();
			for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				ApplyFillWithMask(LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex), MaskColor, FillColor);
			}
		}

		BeginInitResource(&LODData.StaticVertexBuffers.ColorVertexBuffer);
	}	

	checkf(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex), TEXT("Invalid Imported Model index for vertex painting"));
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	const uint32 NumVertices = LODModel.NumVertices;
	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODModel.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
		/** If there is an actual mask apply it to Fill Color when changing the per-vertex color */
		ApplyFillWithMask(LODModel.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color, MaskColor, FillColor);
	}

	if (!SkeletalMesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors)
	{
		SkeletalMesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors = true;
	}
}

void UMeshPaintingSubsystem::ApplyFillWithMask(FColor& InOutColor, const FColor& MaskColor, const FColor& FillColor)
{
	InOutColor.R = ((InOutColor.R & (~MaskColor.R)) | (FillColor.R & MaskColor.R));
	InOutColor.G = ((InOutColor.G & (~MaskColor.G)) | (FillColor.G & MaskColor.G));
	InOutColor.B = ((InOutColor.B & (~MaskColor.B)) | (FillColor.B & MaskColor.B));
	InOutColor.A = ((InOutColor.A & (~MaskColor.A)) | (FillColor.A & MaskColor.A));
}


void UMeshPaintingSubsystem::ForceRenderMeshLOD(UMeshComponent* Component, int32 LODIndex)
{
	// This seems dangerous. What if we save the actor while in forced LOD mode? What if we are stomping an art intended forced LOD?
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		StaticMeshComponent->SetForcedLodModel(LODIndex + 1);
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
	{
		SkeletalMeshComponent->SetForcedLOD(LODIndex + 1);
	}	
}

void UMeshPaintingSubsystem::ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, UMeshComponent* InMeshComponent)
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InMeshComponent))
	{
		ApplyVertexColorsToAllLODs(GeometryInfo, StaticMeshComponent);
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InMeshComponent))
	{
		ApplyVertexColorsToAllLODs(GeometryInfo, SkeletalMeshComponent);
	}
}

void UMeshPaintingSubsystem::ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, UStaticMeshComponent* StaticMeshComponent)
{
	// If a static mesh component was found, apply LOD0 painting to all lower LODs.
	if (!StaticMeshComponent || !StaticMeshComponent->GetStaticMesh())
	{
		return;
	}

	if (StaticMeshComponent->LODData.Num() < 1)
	{
		//We need at least some painting on the base LOD to apply it to the lower LODs
		return;
	}

	//Make sure we have something paint in the LOD 0 to apply it to all lower LODs.
	if (StaticMeshComponent->LODData[0].OverrideVertexColors == nullptr && StaticMeshComponent->LODData[0].PaintedVertices.Num() <= 0)
	{
		return;
	}

	StaticMeshComponent->bCustomOverrideVertexColorPerLOD = false;

	uint32 NumLODs = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources.Num();
	StaticMeshComponent->Modify();

	// Ensure LODData has enough entries in it, free not required.
	StaticMeshComponent->SetLODDataCount(NumLODs, StaticMeshComponent->LODData.Num());
	for (uint32 i = 1; i < NumLODs; ++i)
	{
		FStaticMeshComponentLODInfo* CurrInstanceMeshLODInfo = &StaticMeshComponent->LODData[i];
		FStaticMeshLODResources& CurrRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[i];
		// Destroy the instance vertex  color array if it doesn't fit
		if (CurrInstanceMeshLODInfo->OverrideVertexColors
			&& CurrInstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() != CurrRenderData.GetNumVertices())
		{
			CurrInstanceMeshLODInfo->ReleaseOverrideVertexColorsAndBlock();
		}

		if (CurrInstanceMeshLODInfo->OverrideVertexColors)
		{
			CurrInstanceMeshLODInfo->BeginReleaseOverrideVertexColors();
		}
		else
		{
			// Setup the instance vertex color array if we don't have one yet
			CurrInstanceMeshLODInfo->OverrideVertexColors = new FColorVertexBuffer;
		}
	}

	FlushRenderingCommands();
	
	const FStaticMeshComponentLODInfo& SourceCompLODInfo = StaticMeshComponent->LODData[0];
	const FStaticMeshLODResources& SourceRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[0];
	for (uint32 i = 1; i < NumLODs; ++i)
	{
		FStaticMeshComponentLODInfo& CurCompLODInfo = StaticMeshComponent->LODData[i];
		FStaticMeshLODResources& CurRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[i];

		check(CurCompLODInfo.OverrideVertexColors);
		check(SourceCompLODInfo.OverrideVertexColors);

		TArray<FColor> NewOverrideColors;
		
		RemapPaintedVertexColors(
			SourceCompLODInfo.PaintedVertices,
			SourceCompLODInfo.OverrideVertexColors,
			SourceRenderData.VertexBuffers.PositionVertexBuffer,
			SourceRenderData.VertexBuffers.StaticMeshVertexBuffer,
			CurRenderData.VertexBuffers.PositionVertexBuffer,
			&CurRenderData.VertexBuffers.StaticMeshVertexBuffer,
			NewOverrideColors
		);
		
		if (NewOverrideColors.Num())
		{
			CurCompLODInfo.OverrideVertexColors->InitFromColorArray(NewOverrideColors);
		}

		// Initialize the vert. colors
		BeginInitResource(CurCompLODInfo.OverrideVertexColors);
	}
}

bool UMeshPaintingSubsystem::TryGetNumberOfLODs(const UMeshComponent* MeshComponent, int32& OutNumLODs)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh != nullptr)
		{
			OutNumLODs = StaticMesh->GetNumLODs();
			return true;
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr)
		{
			OutNumLODs = SkeletalMesh->GetLODNum();
			return true;
		}
	}

	return false;
}

int32 UMeshPaintingSubsystem::GetNumberOfLODs(const UMeshComponent* MeshComponent)
{
	int32 NumLODs = 1;
	TryGetNumberOfLODs(MeshComponent, NumLODs);
	return NumLODs;
}

int32 UMeshPaintingSubsystem::GetNumberOfUVs(const UMeshComponent* MeshComponent, int32 LODIndex) const
{
	int32 NumUVs = 0;

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh != nullptr && StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex))
		{
			NumUVs = StaticMesh->GetRenderData()->LODResources[LODIndex].GetNumTexCoords();
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr && SkeletalMesh->GetResourceForRendering() && SkeletalMesh->GetResourceForRendering()->LODRenderData.IsValidIndex(LODIndex))
		{
			NumUVs = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].GetNumTexCoords();
		}
	}

	return NumUVs;
}

bool UMeshPaintingSubsystem::DoesMeshComponentContainPerLODColors(const UMeshComponent* MeshComponent)
{
	bool bPerLODColors = false;

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
	{
		bPerLODColors = StaticMeshComponent->bCustomOverrideVertexColorPerLOD;

		bool bInstancedLODColors = false;
		if (bPerLODColors)
		{

			const int32 NumLODs = StaticMeshComponent->LODData.Num();

			for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
			{
				if (StaticMeshComponent->LODData[LODIndex].PaintedVertices.Num() > 0)
				{
					bInstancedLODColors = true;
					break;
				}
			}
		}

		bPerLODColors = bPerLODColors && bInstancedLODColors;
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent))
	{
		if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			// Only check LOD level 1 and above
			for (int32 LODIndex = 1, NumLODs = SkeletalMesh->GetLODNum(); LODIndex < NumLODs; ++LODIndex)
			{
				const FSkeletalMeshLODInfo* Info = SkeletalMesh->GetLODInfo(LODIndex);
				if (Info->bHasPerLODVertexColors)
				{
					bPerLODColors = true;
					break;
				}
			}
		}
	}

	return bPerLODColors;
}

void UMeshPaintingSubsystem::GetInstanceColorDataInfo(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, int32& OutTotalInstanceVertexColorBytes)
{
	checkf(StaticMeshComponent, TEXT("Invalid StaticMeshComponent"));
	OutTotalInstanceVertexColorBytes = 0;
	
	const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh != nullptr && StaticMesh->GetNumLODs() > (int32)LODIndex && StaticMeshComponent->LODData.IsValidIndex(LODIndex))
	{
		// count the instance color data		
		const FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[LODIndex];
		if (InstanceMeshLODInfo.OverrideVertexColors)
		{
			OutTotalInstanceVertexColorBytes += InstanceMeshLODInfo.OverrideVertexColors->GetAllocatedSize();
		}
	}
}

FColor UMeshPaintingSubsystem::PickVertexColorFromTextureData(const uint8* MipData, const FVector2D& UVCoordinate, const UTexture2D* Texture, const FColor ColorMask)
{
	checkf(MipData, TEXT("Invalid texture MIP data"));
	FColor VertexColor = FColor::Black;

	if ((UVCoordinate.X >= 0.0f) && (UVCoordinate.X < 1.0f) && (UVCoordinate.Y >= 0.0f) && (UVCoordinate.Y < 1.0f))
	{
		const int32 X = Texture->GetSizeX()*UVCoordinate.X;
		const int32 Y = Texture->GetSizeY()*UVCoordinate.Y;

		const int32 Index = ((Y * Texture->GetSizeX()) + X) * 4;
		VertexColor.B = MipData[Index + 0];
		VertexColor.G = MipData[Index + 1];
		VertexColor.R = MipData[Index + 2];
		VertexColor.A = MipData[Index + 3];

		VertexColor.DWColor() &= ColorMask.DWColor();
	}

	// Vertex color is linear
 	if (Texture->SRGB)
 	{
 		VertexColor = FLinearColor(VertexColor).ToFColor(false);
 	}

	return VertexColor;
}


bool UMeshPaintingSubsystem::GetPerVertexPaintInfluencedVertices(FPerVertexPaintActionArgs& InArgs, TSet<int32>& InfluencedVertices)
{
	// Retrieve components world matrix
	const FMatrix& ComponentToWorldMatrix = InArgs.Adapter->GetComponentToWorldMatrix();
	
	// Compute the camera position in actor space.  We need this later to check for back facing triangles.
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(InArgs.HitResult.Location));

	// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
	const float BrushRadius = InArgs.BrushProperties->BrushRadius;
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushRadius, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Get a list of unique vertices indexed by the influenced triangles
	InArgs.Adapter->GetInfluencedVertexIndices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, InArgs.BrushProperties->bOnlyFrontFacingTriangles, InfluencedVertices);

	return (InfluencedVertices.Num() > 0);
}

bool UMeshPaintingSubsystem::ApplyPerVertexPaintAction(FPerVertexPaintActionArgs& InArgs, FPerVertexPaintAction Action)
{
	// Get a list of unique vertices indexed by the influenced triangles
	TSet<int32> InfluencedVertices;
	GetPerVertexPaintInfluencedVertices(InArgs, InfluencedVertices);

	if (InfluencedVertices.Num())
	{
		InArgs.Adapter->PreEdit();
		for (const int32 VertexIndex : InfluencedVertices)
		{
		// Apply the action!			
			Action.ExecuteIfBound(InArgs, VertexIndex);
		}
		InArgs.Adapter->PostEdit();
	}

	return (InfluencedVertices.Num() > 0);
}

bool UMeshPaintingSubsystem::ApplyPerTrianglePaintAction(IMeshPaintComponentAdapter* Adapter, const FVector& CameraPosition, const FVector& HitPosition, const UBrushBaseProperties* Settings, FPerTrianglePaintAction Action, bool bOnlyFrontFacingTriangles)
{
	// Retrieve components world matrix
	const FMatrix& ComponentToWorldMatrix = Adapter->GetComponentToWorldMatrix();

	// Compute the camera position in actor space.  We need this later to check for back facing triangles.
	const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(CameraPosition));
	const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(HitPosition));

	// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
	const float BrushRadius = Settings->BrushRadius;
	const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushRadius, 0.0f, 0.0f)).Size();
	const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

	// Get a list of (optionally front-facing) triangles that are within a reasonable distance to the brush
	TArray<uint32> InfluencedTriangles = Adapter->SphereIntersectTriangles(
		ComponentSpaceSquaredBrushRadius,
		ComponentSpaceBrushPosition,
		ComponentSpaceCameraPosition,
		bOnlyFrontFacingTriangles);

	int32 TriangleIndices[3];

	const TArray<uint32> VertexIndices = Adapter->GetMeshIndices();
	for (uint32 TriangleIndex : InfluencedTriangles)
	{
		// Grab the vertex indices and points for this triangle
		for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
		{
			TriangleIndices[TriVertexNum] = VertexIndices[TriangleIndex * 3 + TriVertexNum];
		}

		Action.Execute(Adapter, TriangleIndex, TriangleIndices);
	}

	return (InfluencedTriangles.Num() > 0);
}

struct FPaintedMeshVertex
{
	FVector Position;
	FPackedNormal Normal;
	FColor Color;
};

/** Helper struct for the mesh component vert position octree */
struct FVertexColorPropogationOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	 * Get the bounding box of the provided octree element. In this case, the box
	 * is merely the point specified by the element.
	 *
	 * @param	Element	Octree element to get the bounding box for
	 *
	 * @return	Bounding box of the provided octree element
	 */
	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox( const FPaintedMeshVertex& Element )
	{
		return FBoxCenterAndExtent( Element.Position, FVector::ZeroVector );
	}

	/**
	 * Determine if two octree elements are equal
	 *
	 * @param	A	First octree element to check
	 * @param	B	Second octree element to check
	 *
	 * @return	true if both octree elements are equal, false if they are not
	 */
	FORCEINLINE static bool AreElementsEqual( const FPaintedMeshVertex& A, const FPaintedMeshVertex& B )
	{
		return ( A.Position == B.Position && A.Normal == B.Normal && A.Color == B.Color );
	}

	/** Ignored for this implementation */
	FORCEINLINE static void SetElementId( const FPaintedMeshVertex& Element, FOctreeElementId2 Id )
	{
	}
};
typedef TOctree2<FPaintedMeshVertex, FVertexColorPropogationOctreeSemantics> TVertexColorPropogationPosOctree;

void UMeshPaintingSubsystem::ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, USkeletalMeshComponent* SkeletalMeshComponent)
{
	checkf(SkeletalMeshComponent != nullptr, TEXT("Invalid Skeletal Mesh Component"));
	USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (Mesh)
	{
		FSkeletalMeshRenderData* Resource = Mesh->GetResourceForRendering();
		FSkeletalMeshModel* SrcMesh = Mesh->GetImportedModel();
		if (Resource)
		{
			const int32 NumLODs = Resource->LODRenderData.Num();
			if (NumLODs > 1)
			{
				const FSkeletalMeshLODRenderData& BaseLOD = Resource->LODRenderData[0];
				GeometryInfo.PreEdit();				

				PropagateVertexPaintToSkeletalMesh(Mesh, 0);

				FBox BaseBounds(ForceInitToZero);

				TArray<FPaintedMeshVertex> PaintedVertices;
				PaintedVertices.Empty(BaseLOD.GetNumVertices());

				FPaintedMeshVertex PaintedVertex;
				for (uint32 VertexIndex = 0; VertexIndex < BaseLOD.GetNumVertices(); ++VertexIndex )
				{
					const FVector VertexPos = (FVector)BaseLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);

					FPackedNormal VertexTangentX, VertexTangentZ;
					VertexTangentX = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexIndex);
					VertexTangentZ = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);

					BaseBounds += VertexPos;
					PaintedVertex.Position = VertexPos;
					PaintedVertex.Normal = VertexTangentZ;
					PaintedVertex.Color = BaseLOD.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
					PaintedVertices.Add(PaintedVertex);
				}

				for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
				{
					// Do something
					FSkeletalMeshLODRenderData& ApplyLOD = Resource->LODRenderData[LODIndex];
					FSkeletalMeshLODModel& SrcLOD = SrcMesh->LODModels[LODIndex];

					FBox CombinedBounds = BaseBounds;
					Mesh->GetLODInfo(LODIndex)->bHasPerLODVertexColors = false;

					if (!ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.IsInitialized())
					{
						ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor::White, ApplyLOD.GetNumVertices());
					}

					for (uint32 VertIndex=0; VertIndex<ApplyLOD.GetNumVertices(); VertIndex++)
					{
						const FVector VertexPos = (FVector)ApplyLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertIndex);
						CombinedBounds += VertexPos;
					}
					
					TVertexColorPropogationPosOctree VertPosOctree(CombinedBounds.GetCenter(), CombinedBounds.GetExtent().GetMax());
					
					// Add each old vertex to the octree
					for (const FPaintedMeshVertex& Vertex : PaintedVertices)
					{
						VertPosOctree.AddElement(Vertex);
					}

					// Iterate over each new vertex position, attempting to find the old vertex it is closest to, applying
					// the color of the old vertex to the new position if possible.
					const float DistanceOverNormalThreshold = KINDA_SMALL_NUMBER;
					check(SrcLOD.NumVertices == ApplyLOD.GetNumVertices());
					for (uint32 VertexIndex = 0; VertexIndex < ApplyLOD.GetNumVertices(); ++VertexIndex)
					{
						TArray<FPaintedMeshVertex> PointsToConsider;
						const FVector CurPosition = (FVector)ApplyLOD.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);

						FPackedNormal VertexTangentZ;
						VertexTangentZ = BaseLOD.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
						
						FVector CurNormal = VertexTangentZ.ToFVector();

						// Iterate through the octree attempting to find the vertices closest to the current new point
						VertPosOctree.FindNearbyElements(CurPosition, [&PointsToConsider](const FPaintedMeshVertex& Vertex)
						{
							// Add all of the elements in the current node to the list of points to consider for closest point calculations
							PointsToConsider.Add(Vertex);
						});

						// If any points to consider were found, iterate over each and find which one is the closest to the new point 
						if (PointsToConsider.Num() > 0)
						{
							int32 BestVertexIndex = 0;
							FVector BestVertexNormal = PointsToConsider[BestVertexIndex].Normal.ToFVector();

							float BestDistanceSquared = (PointsToConsider[BestVertexIndex].Position - CurPosition).SizeSquared();
							float BestNormalDot = BestVertexNormal | CurNormal;

							for (int32 ConsiderationIndex = 1; ConsiderationIndex < PointsToConsider.Num(); ++ConsiderationIndex)
							{
								FPaintedMeshVertex& CheckVertex = PointsToConsider[ConsiderationIndex];
								FVector VertexNormal = CheckVertex.Normal.ToFVector();

								const float DistSqrd = (CheckVertex.Position - CurPosition).SizeSquared();
								const float NormalDot = VertexNormal | CurNormal;
								if (DistSqrd < BestDistanceSquared - DistanceOverNormalThreshold)
								{
									BestVertexIndex = ConsiderationIndex;
									
									BestDistanceSquared = DistSqrd;
									BestNormalDot = NormalDot;
								}
								else if (DistSqrd < BestDistanceSquared + DistanceOverNormalThreshold && NormalDot > BestNormalDot)
								{
									BestVertexIndex = ConsiderationIndex;
									BestDistanceSquared = DistSqrd;
									BestNormalDot = NormalDot;
								}
							}

							ApplyLOD.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = PointsToConsider[BestVertexIndex].Color;
							// Also apply to the skeletal mesh source mesh
							int32 SectionIndex = INDEX_NONE;
							int32 SectionVertexIndex = INDEX_NONE;
							SrcLOD.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
							SrcLOD.Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = PointsToConsider[BestVertexIndex].Color;
						}
					}
					PropagateVertexPaintToSkeletalMesh(Mesh, LODIndex);
				}
				
				GeometryInfo.PostEdit();
			}
		}
	}
}

TSharedPtr<IMeshPaintComponentAdapter> UMeshPaintingSubsystem::GetAdapterForComponent(const UMeshComponent* InComponent) const
{
	return InComponent ? ComponentToAdapterMap.FindRef(InComponent->GetPathName()) : TSharedPtr<IMeshPaintComponentAdapter>{};
}

void UMeshPaintingSubsystem::AddToComponentToAdapterMap(const UMeshComponent* InComponent, const TSharedPtr<IMeshPaintComponentAdapter> InAdapter) 
{
	if (!InAdapter || !InComponent)
	{
		return;
	}

	ComponentToAdapterMap.Add(InComponent->GetPathName(), InAdapter);
}

TArray<UMeshComponent*> UMeshPaintingSubsystem::GetSelectedMeshComponents() const
{
	TArray<UMeshComponent*> Result;
	Result.Reserve(SelectedMeshComponents.Num());

	for (TWeakObjectPtr<UMeshComponent> SelectedMeshComponent : SelectedMeshComponents)
	{
		if (SelectedMeshComponent.IsValid())
		{
			Result.Add(SelectedMeshComponent.Get());
		}
	}

	return Result;
}

void UMeshPaintingSubsystem::ClearSelectedMeshComponents()
{
	SelectedMeshComponents.Empty();
}


void UMeshPaintingSubsystem::AddSelectedMeshComponents(const TArray<UMeshComponent*>& InComponents)
{
	SelectedMeshComponents.Reserve(SelectedMeshComponents.Num() + InComponents.Num());
	Algo::Copy(InComponents, SelectedMeshComponents);
}


TArray<UMeshComponent*> UMeshPaintingSubsystem::GetPaintableMeshComponents() const
{
	TArray<UMeshComponent*> Result;
	Result.Reserve(PaintableComponents.Num());

	for (TWeakObjectPtr<UMeshComponent> PaintableComponent : PaintableComponents)
	{
		if (PaintableComponent.IsValid())
		{
			Result.Add(PaintableComponent.Get());
		}
	}

	return Result;
}

void UMeshPaintingSubsystem::AddPaintableMeshComponent(UMeshComponent* InComponent)
{
	PaintableComponents.Add(InComponent);
}

void UMeshPaintingSubsystem::ClearPaintableMeshComponents()
{
	PaintableComponents.Empty();
}

void UMeshPaintingSubsystem::ResetState()
{
	PaintableComponents.Empty();
	SelectedMeshComponents.Empty();
}

void UMeshPaintingSubsystem::Refresh()
{
	// Ensure that we call OnRemoved while adapter/components are still valid
	PaintableComponents.Empty();
	CleanUp();

	bNeedsRecache = true;
}

void UMeshPaintingSubsystem::CleanUp()
{
	LastPaintedComponent = nullptr;

	for (auto& MeshAdapterPair : ComponentToAdapterMap)
	{
		MeshAdapterPair.Value->OnRemoved();
	}
	ComponentToAdapterMap.Empty();
	FMeshPaintComponentAdapterFactory::CleanupGlobals();
}

bool UMeshPaintingSubsystem::FindHitResult(const FRay Ray, FHitResult& BestTraceResult)
{
	const FVector& Origin = Ray.Origin;
	const FVector& Direction = Ray.Direction;
	BestTraceResult.Distance = FLT_MAX;
	// Fire out a ray to see if there is a *selected* component under the mouse cursor that can be painted.
	{
		const FVector TraceStart(Origin);
		const FVector TraceEnd(Origin + Direction * HALF_WORLD_MAX);

		for (UMeshComponent* MeshComponent : GetPaintableMeshComponents())
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = GetAdapterForComponent(MeshComponent);
			if (!MeshAdapter.IsValid())
			{
				continue;
			}

			// Ray trace
			FHitResult TraceHitResult(1.0f);

			if (MeshAdapter->LineTraceComponent(TraceHitResult, TraceStart, TraceEnd, FCollisionQueryParams(SCENE_QUERY_STAT(Paint), true)))
			{
				// Find the closest impact
				if ((BestTraceResult.GetComponent() == nullptr) || (TraceHitResult.Distance < BestTraceResult.Distance))
				{
					BestTraceResult = TraceHitResult;
				}
			}
		}
	}
	return BestTraceResult.Distance != FLT_MAX;
}

void UMeshPaintingSubsystem::UpdatePaintSupportState()
{
	bSelectionSupportsVertexPaint = false;
	bSelectionSupportsTextureColorPaint = false;
	bSelectionSupportsTextureAssetPaint = false;

	TArray<FPaintableTexture> CommonPaintableTextures;
	bool bCommonPaintableTexturesInitialized = false;

	for (TWeakObjectPtr<UMeshComponent> MeshComponentWeak : SelectedMeshComponents)
	{
		if (UMeshComponent* MeshComponent = MeshComponentWeak.Get())
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = GetAdapterForComponent(MeshComponent);
			if (!MeshAdapter.IsValid() || !MeshAdapter->IsValid())
			{
				MeshAdapter = FMeshPaintComponentAdapterFactory::CreateAdapterForMesh(MeshComponent, 0);
				if (MeshAdapter)
				{
					AddToComponentToAdapterMap(MeshComponent, MeshAdapter);
				}
			}

			if (MeshAdapter)
			{
				bSelectionSupportsVertexPaint |= MeshAdapter->SupportsVertexPaint();

				const bool bSupportsTextureColorPaint = MeshAdapter->SupportsTextureColorPaint();
				const bool bSupportsTextureAssetPaint = MeshAdapter->SupportsTexturePaint();

				if (bSupportsTextureColorPaint || bSupportsTextureAssetPaint)
				{
					// Collect PaintableTextures. This if for both TextureColor painting (MeshPaintTextures on components) and TextureAsset painting (Textures ref'd in materials).
					int32 DummyDefaultIndex = INDEX_NONE;
					TArray<FPaintableTexture> PaintableTextures;
					UTexturePaintToolset::RetrieveTexturesForComponent(MeshComponent, MeshAdapter.Get(), DummyDefaultIndex, PaintableTextures);

					for (FPaintableTexture const& PaintableTexture : PaintableTextures)
					{
						// The bIsMeshTexture tells us which mode the paintable texture works with.
						bSelectionSupportsTextureColorPaint |= bSupportsTextureColorPaint && PaintableTexture.bIsMeshTexture;
						bSelectionSupportsTextureAssetPaint |= bSupportsTextureAssetPaint && !PaintableTexture.bIsMeshTexture;
					}

					if (bSupportsTextureAssetPaint)
					{
						// Fill CommonPaintableTextures array with textures that are in present in ALL components.
						if (!bCommonPaintableTexturesInitialized)
						{
							CommonPaintableTextures = MoveTemp(PaintableTextures);
							bCommonPaintableTexturesInitialized = true;
						}
						else
						{
							TArray<FPaintableTexture> NewCommonPaintableTextures;
							for (FPaintableTexture const& PaintableTexture : PaintableTextures)
							{
								if (CommonPaintableTextures.Contains(PaintableTexture))
								{
									NewCommonPaintableTextures.Add(PaintableTexture);
								}
							}
							CommonPaintableTextures = MoveTemp(NewCommonPaintableTextures);
						}
					}
				}
			}
		}
	}

	// Texture Asset painting requires some texture that is common to all selected components.
	CommonPaintableTextures.RemoveAll([](FPaintableTexture const& PaintableTexture) { return PaintableTexture.bIsMeshTexture; });
	bSelectionSupportsTextureAssetPaint &= CommonPaintableTextures.Num() > 0;
}

TArray<FPerComponentVertexColorData> UMeshPaintingSubsystem::GetCopiedColorsByComponent() const
{
	return CopiedColorsByComponent;
}

void UMeshPaintingSubsystem::SetCopiedColorsByComponent(TArray<FPerComponentVertexColorData>& InCopiedColors)
{
	CopiedColorsByComponent = InCopiedColors;
}

FImage const& UMeshPaintingSubsystem::GetCopiedTexture() const
{
	return CopiedTextureData;
}

void UMeshPaintingSubsystem::SetCopiedTexture(UTexture* InTexture)
{
	if (InTexture != nullptr)
	{
		InTexture->Source.GetMipImage(CopiedTextureData, 0);
	}
}

void UMeshPaintingSubsystem::CacheSelectionData(const int32 PaintLODIndex, const int32 UVChannel)
{
	bSelectionContainsPerLODColors = false;
	for (UMeshComponent* MeshComponent : GetSelectedMeshComponents())
	{
		// Don't create an adapter for transient components, as it doesn't make sense to edit them if they will not be saved.
		// (An example is the procedurally generated foliage meshes)
		if (!MeshComponent->HasAnyFlags(RF_Transient))
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = FMeshPaintComponentAdapterFactory::CreateAdapterForMesh(MeshComponent, PaintLODIndex);
			if (MeshComponent->IsVisible() && MeshAdapter.IsValid() && MeshAdapter->IsValid())
			{
				TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
				AddPaintableMeshComponent(MeshComponent);
				AddToComponentToAdapterMap(MeshComponent, MeshAdapter);
				MeshAdapter->OnAdded();
				GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->ForceRenderMeshLOD(MeshComponent, PaintLODIndex);
				ComponentReregisterContext.Reset(new FComponentReregisterContext(MeshComponent));
				bSelectionContainsPerLODColors |= GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->DoesMeshComponentContainPerLODColors(MeshComponent);
			}
		}
	}
	bNeedsRecache = false;
}

FIntPoint UMeshPaintingSubsystem::GetMinMaxUVChannelsToPaint() const
{
	// For multiple selection take the value from the component with fewest UV channels.
	int32 MinNumUVs = INDEX_NONE;
	int32 MaxNumUVs = INDEX_NONE;
	for (TWeakObjectPtr<UMeshComponent> PaintableComponent : PaintableComponents)
	{
		if (UMeshComponent* MeshComponent = PaintableComponent.Get())
		{
			const int32 NumUVs = GetNumberOfUVs(MeshComponent, 0);
			MinNumUVs = MinNumUVs == INDEX_NONE ? NumUVs : FMath::Min(MinNumUVs, NumUVs);
			MaxNumUVs = MaxNumUVs == INDEX_NONE ? NumUVs : FMath::Max(MaxNumUVs, NumUVs);
		}
	}

	MinNumUVs = MinNumUVs == INDEX_NONE ? 0 : MinNumUVs;
	MaxNumUVs = MaxNumUVs == INDEX_NONE ? 0 : MaxNumUVs;
	return FIntPoint(MinNumUVs, MaxNumUVs);
}

TCHAR const* FMeshPaintToolSettingHelpers::GetCacheIdentifier()
{
	return TEXT("MeshPaint");
}

void FMeshPaintToolSettingHelpers::SavePropertiesForClassHeirachy(UInteractiveTool* InTool, UInteractiveToolPropertySet* InProperties)
{
	// Properties are saved in a cache on the class CDO. 
	// So we need to iterate the class structure and save for each class in the hierarchy.
	// That way classes that share a base class will both use the same cache for their shared properties.
	// Specifically we keep the same base class brush settings when switching between the different painting tools.
	const FString CacheIdentifier(GetCacheIdentifier());
	InProperties->SaveProperties(InTool, CacheIdentifier);

	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
	Params.bOnlyHandleDirectSubObjects = true;

	UClass* Class = InProperties->GetClass();
	if (Class != UInteractiveToolPropertySet::StaticClass())
	{
		Class = Class->GetSuperClass();
		while (Class != UInteractiveToolPropertySet::StaticClass())
		{
			UInteractiveToolPropertySet* Properties = NewObject<UInteractiveToolPropertySet>(GetTransientPackage(), Class);
			UEngine::CopyPropertiesForUnrelatedObjects(InProperties, Properties, Params);
			Properties->SaveProperties(InTool, CacheIdentifier);
			Class = Class->GetSuperClass();
		}
	}
}

void FMeshPaintToolSettingHelpers::RestorePropertiesForClassHeirachy(UInteractiveTool* InTool, UInteractiveToolPropertySet* InProperties)
{
	// Restore from leaf class down the hierarchy, with each base class overriding what was written before.
	const FString CacheIdentifier(GetCacheIdentifier());
	InProperties->RestoreProperties(InTool, CacheIdentifier);

	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
	Params.bOnlyHandleDirectSubObjects = true;

	UClass* Class = InProperties->GetClass();
	if (Class != UInteractiveToolPropertySet::StaticClass())
	{
		Class = Class->GetSuperClass();
		while (Class != UInteractiveToolPropertySet::StaticClass())
		{
			UInteractiveToolPropertySet* Properties = NewObject<UInteractiveToolPropertySet>(GetTransientPackage(), Class);
			Properties->RestoreProperties(InTool, CacheIdentifier);
			UEngine::CopyPropertiesForUnrelatedObjects(Properties, InProperties, Params);
			Class = Class->GetSuperClass();
		}
	}
}
