// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPaintInteractions.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "ToolContextInterfaces.h"
#include "MeshPaintHelpers.h"
#include "Components/MeshComponent.h"
#include "IMeshPaintComponentAdapter.h"
#include "MeshPaintAdapterFactory.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintInteractions)

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "MeshSelection"

FInputRayHit UMeshPaintSelectionMechanic::IsHitByClick(const FInputDeviceRay& ClickPos, bool bIsFallbackClick)
{
	IMeshPaintSelectionInterface* Interface = Cast<IMeshPaintSelectionInterface>(GetParentTool());
	if (!bAddToSelectionSet || !Interface->AllowsMultiselect())
	{
		CachedClickedComponents.Empty();
		CachedClickedActors.Empty();
	}

	// for fallback clicks, assume we must be adding to our selection set
	if (bIsFallbackClick)
	{
		if (Interface->AllowsMultiselect() && bAddToSelectionSet)
		{
			return FindClickedComponentsAndCacheAdapters(ClickPos) ? FInputRayHit(0.0f) : FInputRayHit();
		}
		else
		{
			return FInputRayHit();
		}
	}
	return FindClickedComponentsAndCacheAdapters(ClickPos) ? FInputRayHit(0.0f) : FInputRayHit();
}

void UMeshPaintSelectionMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		for (UMeshComponent* MeshComponent : CachedClickedComponents)
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter;
			if (MeshComponent)
			{
				MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);
			}
			IMeshPaintSelectionInterface* Interface = Cast<IMeshPaintSelectionInterface>(GetParentTool());
			if (MeshComponent && MeshComponent->IsVisible() && MeshAdapter.IsValid() && MeshAdapter->IsValid() && Interface->IsMeshAdapterSupported(MeshAdapter))
			{
				MeshPaintingSubsystem->AddPaintableMeshComponent(MeshComponent);
				MeshAdapter->OnAdded();
			}

			GetParentTool()->GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshSelection", "Select Mesh"));


			FSelectedObjectsChangeList NewSelection;
			// TODO add CTRL handling
			const bool bShouldAddToSelection = bAddToSelectionSet && Interface->AllowsMultiselect();
			NewSelection.ModificationType = bShouldAddToSelection ? ESelectedObjectsModificationType::Add : ESelectedObjectsModificationType::Replace;
			NewSelection.Actors.Append(CachedClickedActors);
			GetParentTool()->GetToolManager()->RequestSelectionChange(NewSelection);
			GetParentTool()->GetToolManager()->EndUndoTransaction();
		}
	}
}

bool UMeshPaintSelectionMechanic::FindClickedComponentsAndCacheAdapters(const FInputDeviceRay& ClickPos)
{
	bool bFoundValidComponents = false;
#if WITH_EDITOR
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (!MeshPaintingSubsystem)
	{
		return bFoundValidComponents;
	}

	FViewport* FocusedViewport = GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (FocusedViewport)
	{
		if (HHitProxy* HitProxy = FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y))
		{
			if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(HitProxy->GetElementHandle()))
			{
				if (UActorComponent* ObjectInterfaceActorComponent = ObjectInterface.GetObjectAs<UActorComponent>())
				{
					if (AActor* Actor = ObjectInterfaceActorComponent->GetOwner())
					{
						TArray<UActorComponent*> CandidateComponents = Actor->K2_GetComponentsByClass(UMeshComponent::StaticClass());
						for (UActorComponent* CandidateComponent : CandidateComponents)
						{
							UMeshComponent* MeshComponent = Cast<UMeshComponent>(CandidateComponent);
							TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);
							if (!MeshAdapter.IsValid() || !MeshAdapter->IsValid())
							{
								MeshAdapter = FMeshPaintComponentAdapterFactory::CreateAdapterForMesh(MeshComponent, 0);
								if (MeshAdapter)
								{
									MeshPaintingSubsystem->AddToComponentToAdapterMap(MeshComponent, MeshAdapter);
								}
							}
							IMeshPaintSelectionInterface* Interface = Cast<IMeshPaintSelectionInterface>(GetParentTool());
							if (MeshAdapter.IsValid() && Interface->IsMeshAdapterSupported(MeshAdapter))
							{
								CachedClickedComponents.AddUnique(MeshComponent);
								CachedClickedActors.AddUnique(Cast<AActor>(MeshComponent->GetOuter()));
								bFoundValidComponents = true;
							}
						}
					}
				}
			}
		}
	}
#endif
	return bFoundValidComponents;
}


#undef LOCTEXT_NAMESPACE
