// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintSkeletalMeshAdapter.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "StaticMeshAttributes.h"
#include "Rendering/RenderCommandPipes.h"

DEFINE_LOG_CATEGORY_STATIC(LogMeshPaintSkeletalMeshAdapter, Log, All);
//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshes



//HACK for 4.24.2 we cannot change public API so we use this global function to remap and propagate the vertex color data to the imported model when the user release the mouse
void PropagateVertexPaintToSkeletalMesh(USkeletalMesh* SkeletalMesh, int32 LODIndex)
{
	struct FMatchFaceData
	{
		int32 SoftVertexIndexes[3];
	};

	if (!SkeletalMesh || !SkeletalMesh->HasMeshDescription(LODIndex))
	{
		// We do not propagate vertex color for LODs that don't have editable mesh data.
		return;
	}

	auto GetMatchKey = [](const FVector3f& PositionA, const FVector3f& PositionB, const FVector3f& PositionC)->FSHAHash
	{
		FSHA1 SHA;
		FSHAHash SHAHash;

		SHA.Update(reinterpret_cast<const uint8*>(&PositionA), sizeof(FVector3f));
		SHA.Update(reinterpret_cast<const uint8*>(&PositionB), sizeof(FVector3f));
		SHA.Update(reinterpret_cast<const uint8*>(&PositionC), sizeof(FVector3f));
		SHA.Final();
		SHA.GetHash(&SHAHash.Hash[0]);

		return SHAHash;
	};

	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	const TArray<uint32>& SrcIndexBuffer = LODModel.IndexBuffer;

	TArray<FSoftSkinVertex> SrcVertices;
	LODModel.GetVertices(SrcVertices);

	FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);

	TMap<FSHAHash, FMatchFaceData> MatchTriangles;
	MatchTriangles.Reserve(MeshDescription->Triangles().Num());

	for (int32 IndexBufferIndex = 0, SrcIndexBufferNum = SrcIndexBuffer.Num(); IndexBufferIndex < SrcIndexBufferNum; IndexBufferIndex += 3)
	{
		const FVector3f& PositionA = SrcVertices[SrcIndexBuffer[IndexBufferIndex]].Position;
		const FVector3f& PositionB = SrcVertices[SrcIndexBuffer[IndexBufferIndex + 1]].Position;
		const FVector3f& PositionC = SrcVertices[SrcIndexBuffer[IndexBufferIndex + 2]].Position;

		FSHAHash Key = GetMatchKey(PositionA, PositionB, PositionC);
		FMatchFaceData MatchFaceData;
		MatchFaceData.SoftVertexIndexes[0] = SrcIndexBuffer[IndexBufferIndex];
		MatchFaceData.SoftVertexIndexes[1] = SrcIndexBuffer[IndexBufferIndex + 1];
		MatchFaceData.SoftVertexIndexes[2] = SrcIndexBuffer[IndexBufferIndex + 2];
		MatchTriangles.Add(Key, MatchFaceData);
	}

	FStaticMeshAttributes MeshAttributes(*MeshDescription);
	TVertexInstanceAttributesRef<FVector4f> ColorAttribute = MeshAttributes.GetVertexInstanceColors();

	bool bAllVertexFound = true;
	for (const FTriangleID TriangleID: MeshDescription->Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> TriangleVertexIDs = MeshDescription->GetTriangleVertices(TriangleID);

		FVector3f PositionA = MeshDescription->GetVertexPosition(TriangleVertexIDs[0]);
		FVector3f PositionB = MeshDescription->GetVertexPosition(TriangleVertexIDs[1]);
		FVector3f PositionC = MeshDescription->GetVertexPosition(TriangleVertexIDs[2]);

		const FSHAHash Key = GetMatchKey(PositionA, PositionB, PositionC);
		if (const FMatchFaceData* MatchFaceData = MatchTriangles.Find(Key))
		{
			TArrayView<const FVertexInstanceID> TriangleVertexIndexIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);

			for (int32 Index = 0; Index < 3; Index++)
			{
				FLinearColor Color(SrcVertices[MatchFaceData->SoftVertexIndexes[Index]].Color.ReinterpretAsLinear());
				ColorAttribute.Set(TriangleVertexIndexIDs[Index], Color);
			}
		}
		else if (bAllVertexFound)
		{
			bAllVertexFound = false;
			FString SkeletalMeshName(SkeletalMesh->GetName());
			UE_LOGF(LogMeshPaintSkeletalMeshAdapter, Warning, "Some vertex color data could not be applied to the %ls SkeletalMesh asset.", *SkeletalMeshName);
		}
	}
	
	SkeletalMesh->CommitMeshDescription(LODIndex);
}


bool FMeshPaintSkeletalMeshComponentAdapter::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComponent.IsValid())
	{
		SkeletalMeshChangedHandle = SkeletalMeshComponent->RegisterOnSkeletalMeshPropertyChanged(USkeletalMeshComponent::FOnSkeletalMeshPropertyChanged::CreateSP(this, &FMeshPaintSkeletalMeshComponentAdapter::OnSkeletalMeshChanged));

		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			ReferencedSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
			MeshLODIndex = InMeshLODIndex;
			const bool bSuccess = Initialize();
			return bSuccess;
		}
	}

	return false;
}

FMeshPaintSkeletalMeshComponentAdapter::~FMeshPaintSkeletalMeshComponentAdapter()
{
	if (SkeletalMeshComponent.IsValid())
	{
		SkeletalMeshComponent->UnregisterOnSkeletalMeshPropertyChanged(SkeletalMeshChangedHandle);
	}
	if (ReferencedSkeletalMesh != nullptr)
	{
		ReferencedSkeletalMesh->OnPostMeshCached().RemoveAll(this);
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::OnSkeletalMeshChanged()
{
	OnRemoved();
	if (SkeletalMeshComponent.IsValid())
	{
		ReferencedSkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			Initialize();
			OnAdded();
		}
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::OnPostMeshCached(USkeletalMesh* SkeletalMesh)
{
	if (ReferencedSkeletalMesh == SkeletalMesh)
	{
		OnSkeletalMeshChanged();
	}
}

bool FMeshPaintSkeletalMeshComponentAdapter::Initialize()
{
	bool bInitializationResult = false;

	if (SkeletalMeshComponent.IsValid())
	{
		check(ReferencedSkeletalMesh == SkeletalMeshComponent->GetSkeletalMeshAsset());

		MeshResource = ReferencedSkeletalMesh->GetResourceForRendering();
		if (MeshResource != nullptr)
		{
			LODData = &MeshResource->LODRenderData[MeshLODIndex];
			checkf(ReferencedSkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(MeshLODIndex), TEXT("Invalid Imported Model index for vertex painting"));
			LODModel = &ReferencedSkeletalMesh->GetImportedModel()->LODModels[MeshLODIndex];

			bInitializationResult = FBaseMeshPaintComponentAdapter::Initialize();
		}

	}
	
	return bInitializationResult;
}

bool FMeshPaintSkeletalMeshComponentAdapter::InitializeVertexData()
{
	// Retrieve mesh vertex and index data 
	const int32 NumVertices = LODData->GetNumVertices();
	MeshVertices.Reset();
	MeshVertices.AddDefaulted(NumVertices);
	for (int32 Index = 0; Index < NumVertices; Index++)
	{
		const FVector& Position = (FVector)LODData->StaticVertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		MeshVertices[Index] = Position;
	}

	MeshIndices.Reserve(LODData->MultiSizeIndexContainer.GetIndexBuffer()->Num());
	LODData->MultiSizeIndexContainer.GetIndexBuffer(MeshIndices);

	return (MeshVertices.Num() >= 0 && MeshIndices.Num() > 0);
}

void FMeshPaintSkeletalMeshComponentAdapter::InitializeAdapterGlobals()
{
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{
}

void FMeshPaintSkeletalMeshComponentAdapter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ReferencedSkeletalMesh);
}

void FMeshPaintSkeletalMeshComponentAdapter::CleanupGlobals()
{
}

void FMeshPaintSkeletalMeshComponentAdapter::OnAdded()
{
	// We shouldn't assume that the cached skeletal mesh component remains valid.
	// Components may be destroyed by editor ticks, and be forcibly removed by GC.
	if (!SkeletalMeshComponent.IsValid())
	{
		return;
	}

	checkf(ReferencedSkeletalMesh, TEXT("Invalid reference to Skeletal Mesh"));
	checkf(ReferencedSkeletalMesh == SkeletalMeshComponent->GetSkeletalMeshAsset(), TEXT("Referenced Skeletal Mesh does not match one in Component"));

	SkeletalMeshComponent->bUseRefPoseOnInitAnim = true;
	SkeletalMeshComponent->InitAnim(true);

	// Register callback for when the skeletal mesh is cached underneath us
	ReferencedSkeletalMesh->OnPostMeshCached().AddRaw(this, &FMeshPaintSkeletalMeshComponentAdapter::OnPostMeshCached);
}

void FMeshPaintSkeletalMeshComponentAdapter::OnRemoved()
{
	// We shouldn't assume that the cached skeletal mesh component remains valid.
	// Components may be destroyed by editor ticks, and be forcibly removed by GC.
	if (!SkeletalMeshComponent.IsValid())
	{
		return;
	}
	
	// If the referenced skeletal mesh has been destroyed (and nulled by GC), don't try to do anything more.
	// It should be in the process of removing all global geometry adapters if it gets here in this situation.
	if (!ReferencedSkeletalMesh)
	{
		return;
	}
	PropagateVertexPaintToSkeletalMesh(ReferencedSkeletalMesh, MeshLODIndex);
	SkeletalMeshComponent->bUseRefPoseOnInitAnim = false;
	SkeletalMeshComponent->InitAnim(true);

	ReferencedSkeletalMesh->OnPostMeshCached().RemoveAll(this);
}

int32 FMeshPaintSkeletalMeshComponentAdapter::GetNumUVChannels() const
{
	return LODData == nullptr ? 0 : LODData->GetNumTexCoords();
}

bool FMeshPaintSkeletalMeshComponentAdapter::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	if (!SkeletalMeshComponent.IsValid())
	{
		return false;
	}

	const bool bHitBounds = FMath::LineSphereIntersection(Start, End.GetSafeNormal(), (End - Start).SizeSquared(), SkeletalMeshComponent->Bounds.Origin, SkeletalMeshComponent->Bounds.SphereRadius);
	const float SqrRadius = FMath::Square(SkeletalMeshComponent->Bounds.SphereRadius);
	const bool bInsideBounds = (SkeletalMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(Start) <= SqrRadius) || (SkeletalMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(End) <= SqrRadius);

	bool bHitTriangle = false;
	if (bHitBounds || bInsideBounds)
	{
		const FTransform& ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
		const FVector LocalStart = ComponentTransform.InverseTransformPosition(Start);
		const FVector LocalEnd = ComponentTransform.InverseTransformPosition(End);
		float MinDistance = FLT_MAX;
		FVector Intersect;
		FVector Normal;
		UE::Geometry::FIndex3i FoundTriangle;
		FVector HitPosition;
		if (!RayIntersectAdapter(FoundTriangle, HitPosition, LocalStart, LocalEnd))
		{
			return false;
		}

		// Compute the normal of the triangle
		const FVector& P0 = MeshVertices[FoundTriangle.A];
		const FVector& P1 = MeshVertices[FoundTriangle.B];
		const FVector& P2 = MeshVertices[FoundTriangle.C];

		const FVector TriNorm = (P1 - P0) ^ (P2 - P0);

		//check collinearity of A,B,C
		if (TriNorm.SizeSquared() > SMALL_NUMBER)
		{
			FVector IntersectPoint;
			FVector HitNormal;
		
			bool bHit = FMath::SegmentTriangleIntersection(LocalStart, LocalEnd, P0, P1, P2, IntersectPoint, HitNormal);

			if (bHit)
			{
				const float Distance = (LocalStart - IntersectPoint).SizeSquared();
				if (Distance < MinDistance)
				{
					MinDistance = Distance;
					Intersect = IntersectPoint;
					Normal = HitNormal;
				}
			}
		}
		

		if (MinDistance != FLT_MAX)
		{
			OutHit.Component = SkeletalMeshComponent;
			OutHit.Normal = ComponentTransform.TransformVector(Normal).GetSafeNormal();
			OutHit.ImpactNormal = OutHit.Normal;
			OutHit.ImpactPoint = ComponentTransform.TransformPosition(Intersect);
			OutHit.Location = OutHit.ImpactPoint;
			OutHit.bBlockingHit = true;
			OutHit.Distance = MinDistance;
			bHitTriangle = true;
		}
	}	

	return bHitTriangle;
}

void FMeshPaintSkeletalMeshComponentAdapter::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	if (SkeletalMeshComponent.IsValid())
	{
		DefaultQueryPaintableTextures(MaterialIndex, SkeletalMeshComponent.Get(), OutDefaultIndex, InOutTextureList);
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const
{
	if (SkeletalMeshComponent.IsValid())
	{
		TextureOverridesState.ApplyOrRemoveTextureOverride(SkeletalMeshComponent.Get(), SourceTexture, OverrideTexture, MaterialUpdateContext);
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	OutTextureCoordinate = FVector2D(LODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, ChannelIndex));
}

void FMeshPaintSkeletalMeshComponentAdapter::PreEdit()
{
	if (!SkeletalMeshComponent.IsValid())
	{
		return;
	}

	FlushRenderingCommands();

	SkeletalMeshComponent->Modify();

	ReferencedSkeletalMesh->SetFlags(RF_Transactional);
	ReferencedSkeletalMesh->Modify();

	ReferencedSkeletalMesh->SetHasVertexColors(true);
	ReferencedSkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());

	// Release the static mesh's resources.
	ReferencedSkeletalMesh->ReleaseResources();

	// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
	// allocated, and potentially accessing the UStaticMesh.
	ReferencedSkeletalMesh->ReleaseResourcesFence.Wait();

	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
	{
		// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
		LODData->StaticVertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor(255, 255, 255, 255), LODData->GetNumVertices());
		ReferencedSkeletalMesh->SetHasVertexColors(true);
		ReferencedSkeletalMesh->SetVertexColorGuid(FGuid::NewGuid());
		BeginInitResource(&LODData->StaticVertexBuffers.ColorVertexBuffer, &UE::RenderCommandPipe::SkeletalMesh);
	}
	//Make sure we change the import data so the re-import do not replace the new data
	if (ReferencedSkeletalMesh->GetAssetImportData())
	{
		UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(ReferencedSkeletalMesh->GetAssetImportData());
		if (ImportData && ImportData->VertexColorImportOption != EVertexColorImportOption::Ignore)
		{
			ImportData->SetFlags(RF_Transactional);
			ImportData->Modify();
			ImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		}

		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(ReferencedSkeletalMesh->GetAssetImportData());
		if (InterchangeAssetImportData)
		{
			TArray<UObject*> Pipelines = InterchangeAssetImportData->GetPipelines();
			for (UObject* PipelineBase : Pipelines)
			{
				UInterchangeGenericAssetsPipeline* GenericAssetPipeline = Cast<UInterchangeGenericAssetsPipeline>(PipelineBase);

				if (GenericAssetPipeline)
				{
					if (GenericAssetPipeline->CommonMeshesProperties && GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption != EInterchangeVertexColorImportOption::IVCIO_Ignore)
					{
						GenericAssetPipeline->SetFlags(RF_Transactional);
						GenericAssetPipeline->Modify();
						GenericAssetPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Ignore;
					}
				}
			}
		}
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::PostEdit()
{
	TUniquePtr< FSkinnedMeshComponentRecreateRenderStateContext > RecreateRenderStateContext = MakeUnique<FSkinnedMeshComponentRecreateRenderStateContext>(ReferencedSkeletalMesh);
	ReferencedSkeletalMesh->InitResources();
	ReferencedSkeletalMesh->GetOnMeshChanged().Broadcast();
}

void FMeshPaintSkeletalMeshComponentAdapter::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
	{
		check((int32)LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > VertexIndex);
		OutColor = LODData->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
	}
}

void FMeshPaintSkeletalMeshComponentAdapter::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	if (LODData->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
	{
		LODData->StaticVertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = Color;

		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;		
		LODModel->GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);
		LODModel->Sections[SectionIndex].SoftVertices[SectionVertexIndex].Color = Color;

		if (!ReferencedSkeletalMesh->GetLODInfo(MeshLODIndex)->bHasPerLODVertexColors)
		{
			ReferencedSkeletalMesh->GetLODInfo(MeshLODIndex)->bHasPerLODVertexColors = true;
		}
	}
}

FMatrix FMeshPaintSkeletalMeshComponentAdapter::GetComponentToWorldMatrix() const
{
	if (!SkeletalMeshComponent.IsValid())
	{
		return FMatrix::Identity;
	}

	return SkeletalMeshComponent->GetComponentToWorld().ToMatrixWithScale();
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshesFactory

TSharedPtr<IMeshPaintComponentAdapter> FMeshPaintSkeletalMeshComponentAdapterFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			TSharedRef<FMeshPaintSkeletalMeshComponentAdapter> Result = MakeShareable(new FMeshPaintSkeletalMeshComponentAdapter());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
