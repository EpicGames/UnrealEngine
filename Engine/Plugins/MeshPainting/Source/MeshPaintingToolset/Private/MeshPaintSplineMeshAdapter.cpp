// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintSplineMeshAdapter.h"

#include "StaticMeshResources.h"
#include "Components/SplineMeshComponent.h"

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshes

bool FMeshPaintSplineMeshComponentAdapter::InitializeVertexData()
{
	if (!StaticMeshComponent.IsValid())
	{
		return false;
	}

	// Cache deformed spline mesh vertices for quick lookup during painting / previewing
	USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(StaticMeshComponent.Get());
	check(SplineMeshComponent);

	bool bValid = false;
	if (LODModel)
	{
		// Retrieve vertex and index data 
		const int32 NumVertices = LODModel->VertexBuffers.PositionVertexBuffer.GetNumVertices();
		MeshVertices.Reset();
		MeshVertices.AddDefaulted(NumVertices);
		
		// Apply spline vertex deformation to each vertex
		for (int32 Index = 0; Index < NumVertices; Index++)
		{
			FVector Position = (FVector)LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(Index);
			const FTransform SliceTransform = SplineMeshComponent->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComponent->ForwardAxis));
			USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComponent->ForwardAxis) = 0;
			MeshVertices[Index] = SliceTransform.TransformPosition(Position);
		}

		const int32 NumIndices = LODModel->IndexBuffer.GetNumIndices();
		MeshIndices.Reset();
		MeshIndices.AddDefaulted(NumIndices);
		const FIndexArrayView ArrayView = LODModel->IndexBuffer.GetArrayView();
		for (int32 Index = 0; Index < NumIndices; Index++)
		{
			MeshIndices[Index] = ArrayView[Index];
		}

		bValid = (MeshVertices.Num() > 0 && MeshIndices.Num() > 0);
	}

	return bValid;
}


//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshesFactory

TSharedPtr<IMeshPaintComponentAdapter> FMeshPaintSplineMeshComponentAdapterFactory::Construct(class UMeshComponent* InComponent, int32 MeshLODIndex) const
{
	if (USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(InComponent))
	{
		if (SplineMeshComponent->GetStaticMesh() != nullptr)
		{
			TSharedRef<FMeshPaintSplineMeshComponentAdapter> Result = MakeShareable(new FMeshPaintSplineMeshComponentAdapter());
			if (Result->Construct(InComponent, MeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
