// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintStaticMeshAdapter.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "ComponentReregisterContext.h"
#include "Engine/StaticMesh.h"

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshes

bool FMeshPaintStaticMeshComponentAdapter::Construct(UMeshComponent* InComponent, int32 InMeshLODIndex)
{
	StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent);
	if (StaticMeshComponent.IsValid())
	{
#if WITH_EDITOR
		StaticMeshComponent->OnStaticMeshChanged().AddRaw(this, &FMeshPaintStaticMeshComponentAdapter::OnStaticMeshChanged);
#endif
		if (StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			ReferencedStaticMesh = StaticMeshComponent->GetStaticMesh();
			MeshLODIndex = InMeshLODIndex;
#if WITH_EDITOR
			ReferencedStaticMesh->OnPostMeshBuild().AddRaw(this, &FMeshPaintStaticMeshComponentAdapter::OnPostMeshBuild);
#endif
			const bool bSuccess = Initialize();
			return bSuccess;
		}
	}

	return false;
}

FMeshPaintStaticMeshComponentAdapter::~FMeshPaintStaticMeshComponentAdapter()
{
#if WITH_EDITOR
	if (StaticMeshComponent.IsValid())
	{
		StaticMeshComponent->OnStaticMeshChanged().RemoveAll(this);
	}
	if (ReferencedStaticMesh != nullptr)
	{
		ReferencedStaticMesh->OnPostMeshBuild().RemoveAll(this);
	}
#endif
}

void FMeshPaintStaticMeshComponentAdapter::OnPostMeshBuild(UStaticMesh* StaticMesh)
{
	check(StaticMesh == ReferencedStaticMesh);
	Initialize();
}

void FMeshPaintStaticMeshComponentAdapter::OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent)
{
	check(StaticMeshComponent == InStaticMeshComponent);
	OnRemoved();
	if (ReferencedStaticMesh)
	{
#if WITH_EDITOR
		ReferencedStaticMesh->OnPostMeshBuild().RemoveAll(this);
#endif
		ReferencedStaticMesh = InStaticMeshComponent->GetStaticMesh();
#if WITH_EDITOR
		ReferencedStaticMesh->OnPostMeshBuild().AddRaw(this, &FMeshPaintStaticMeshComponentAdapter::OnPostMeshBuild);
#endif
		Initialize();
		OnAdded();
	}
}

bool FMeshPaintStaticMeshComponentAdapter::Initialize()
{
	if (StaticMeshComponent.IsValid())
	{
		check(ReferencedStaticMesh == StaticMeshComponent->GetStaticMesh());
		if (MeshLODIndex < ReferencedStaticMesh->GetNumLODs())
		{
			LODModel = &(ReferencedStaticMesh->GetRenderData()->LODResources[MeshLODIndex]);
			return FBaseMeshPaintComponentAdapter::Initialize();
		}
	}

	return false;
}

bool FMeshPaintStaticMeshComponentAdapter::InitializeVertexData()
{
	// Retrieve mesh vertex and index data 
	const int32 NumVertices = LODModel->VertexBuffers.PositionVertexBuffer.GetNumVertices();
	MeshVertices.Reset();
	MeshVertices.AddDefaulted(NumVertices);
	for (int32 Index = 0; Index < NumVertices; Index++)
	{
		const FVector& Position = (FVector)LODModel->VertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		MeshVertices[Index] = Position;
	}

	const int32 NumIndices = LODModel->IndexBuffer.GetNumIndices();
	MeshIndices.Reset();
	MeshIndices.AddDefaulted(NumIndices);
	const FIndexArrayView ArrayView = LODModel->IndexBuffer.GetArrayView();
	for (int32 Index = 0; Index < NumIndices; Index++)
	{
		MeshIndices[Index] = ArrayView[Index];
	}

	return (MeshVertices.Num() > 0 && MeshIndices.Num() > 0);
}

void FMeshPaintStaticMeshComponentAdapter::PostEdit()
{
	// We shouldn't assume that the cached static mesh component remains valid.
	// Components may be destroyed by editor ticks, and be forcibly removed by GC.
	if (!StaticMeshComponent.IsValid())
	{
		return;
	}

	// Lighting does not need to be invalidated when mesh painting
	const bool bUnbuildLighting = false;

	// Recreate all component states using the referenced static mesh
	FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(ReferencedStaticMesh, bUnbuildLighting);
	
	const bool bUsingInstancedVertexColors = true; // Currently we are only painting to instances 
	// Update gpu resource data 
	if (bUsingInstancedVertexColors)
	{
		// We're only changing instanced vertices on this specific mesh component, so we
		// only need to detach our mesh component
		FComponentReregisterContext ComponentReregisterContext(StaticMeshComponent.Get());

		// If LOD is 0, post-edit all LODs. There's currently no way to tell from here
		// if VertexPaintSettings.bPaintOnSpecificLOD is set to true or not.
		const int32 MaxLOD = (MeshLODIndex == 0) ? StaticMeshComponent->LODData.Num() : (MeshLODIndex + 1);
		for (int32 Index = MeshLODIndex; Index < MaxLOD; ++Index)
		{
			BeginInitResource(StaticMeshComponent->LODData[Index].OverrideVertexColors);
		}
	}
	else
	{
		// Reinitialize the static mesh's resources.
		ReferencedStaticMesh->InitResources();
	}
}

void FMeshPaintStaticMeshComponentAdapter::InitializeAdapterGlobals()
{
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
	}
}

void FMeshPaintStaticMeshComponentAdapter::AddReferencedObjectsGlobals(FReferenceCollector& Collector)
{

}

void FMeshPaintStaticMeshComponentAdapter::CleanupGlobals()
{
}

void FMeshPaintStaticMeshComponentAdapter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ReferencedStaticMesh);
}

int32 FMeshPaintStaticMeshComponentAdapter::GetNumUVChannels() const
{
	return LODModel == nullptr ? 0 : LODModel->GetNumTexCoords();
}

bool FMeshPaintStaticMeshComponentAdapter::LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const
{
	if (!StaticMeshComponent.IsValid())
	{
		return false;
	}

	// Ray trace
	const bool bHitBounds = FMath::LineSphereIntersection(Start, End.GetSafeNormal(), (End - Start).SizeSquared(), StaticMeshComponent->Bounds.Origin, StaticMeshComponent->Bounds.SphereRadius);
	const float SqrRadius = FMath::Square(StaticMeshComponent->Bounds.SphereRadius);
	const bool bInsideBounds = (StaticMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(Start) <= SqrRadius) || (StaticMeshComponent->Bounds.ComputeSquaredDistanceFromBoxToPoint(End) <= SqrRadius);

	bool bHitTriangle = false;
	if (bHitBounds || bInsideBounds)
	{
		const FTransform& ComponentTransform = StaticMeshComponent->GetComponentTransform();
		const FVector LocalStart = ComponentTransform.InverseTransformPosition(Start);
		const FVector LocalEnd = ComponentTransform.InverseTransformPosition(End);
		float MinDistance = FLT_MAX;
		FVector Intersect;
		FVector Normal;
		UE::Geometry::FIndex3i FoundTriangle;
		FVector HitPosition;
		if (!RayIntersectAdapter(FoundTriangle, HitPosition, LocalStart, LocalEnd))
		{
			return false;
		}

		// Compute the normal of the triangle
		const FVector& P0 = MeshVertices[FoundTriangle.A];
		const FVector& P1 = MeshVertices[FoundTriangle.B];
		const FVector& P2 = MeshVertices[FoundTriangle.C];

		UE::Geometry::FTriangle3d Triangle((FVector3d)P0, (FVector3d)P1, (FVector3d)P2);
		FVector3d TriNormal = Triangle.Normal();

		//check collinearity of A,B,C
		if (TriNormal.SquaredLength() > (double)SMALL_NUMBER)
		{
			FVector3d RayDirection = ((FVector3d)LocalEnd - (FVector3d)LocalStart);
			FRay3d LocalRay((FVector3d)LocalStart, UE::Geometry::Normalized(RayDirection));
			UE::Geometry::FIntrRay3Triangle3d RayTriIntersection(LocalRay, Triangle);
			if (RayTriIntersection.Find())
			{
				double Distance = RayTriIntersection.RayParameter;
				if (Distance < MinDistance)
				{
					MinDistance = (float)Distance;
					Intersect = (FVector)LocalRay.PointAt(RayTriIntersection.RayParameter);
					Normal = (FVector)TriNormal;
				}
			}
		}

		if (MinDistance != FLT_MAX)
		{
			OutHit.Component = StaticMeshComponent;
			OutHit.Normal = ComponentTransform.TransformVector(Normal).GetSafeNormal();
			OutHit.ImpactNormal = OutHit.Normal;
			OutHit.ImpactPoint = ComponentTransform.TransformPosition(Intersect);
			OutHit.Location = OutHit.ImpactPoint;
			OutHit.bBlockingHit = true;
			OutHit.Distance = MinDistance;
			bHitTriangle = true;
		}
	}

	return bHitTriangle;
}

void FMeshPaintStaticMeshComponentAdapter::QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList)
{
	if (StaticMeshComponent.IsValid())
	{
		DefaultQueryPaintableTextures(MaterialIndex, StaticMeshComponent.Get(), OutDefaultIndex, InOutTextureList);
	}
}

void FMeshPaintStaticMeshComponentAdapter::ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const
{
	if (StaticMeshComponent.IsValid())
	{
		TextureOverridesState.ApplyOrRemoveTextureOverride(StaticMeshComponent.Get(), SourceTexture, OverrideTexture, MaterialUpdateContext);
	}
}

void FMeshPaintStaticMeshComponentAdapter::GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance /*= true*/) const
{
	if (bInstance)
	{
		if (!StaticMeshComponent.IsValid())
		{
			OutColor = FColor::White;
			return;
		}

		FStaticMeshComponentLODInfo* InstanceMeshLODInfo = &StaticMeshComponent->LODData[MeshLODIndex];
		if (!bInstance && LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() == 0)
		{
			// Mesh doesn't have a color vertex buffer yet!  We'll create one now.
			LODModel->VertexBuffers.ColorVertexBuffer.InitFromSingleColor(FColor(255, 255, 255, 255), LODModel->GetNumVertices());

			// @todo MeshPaint: Make sure this is the best place to do this
			BeginInitResource(&LODModel->VertexBuffers.ColorVertexBuffer);
		}

		// Actor mesh component LOD
		const bool bValidInstanceData = InstanceMeshLODInfo
			&& InstanceMeshLODInfo->OverrideVertexColors
			&& InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == LODModel->GetNumVertices();
		if (bValidInstanceData)
		{
			OutColor = InstanceMeshLODInfo->OverrideVertexColors->VertexColor(VertexIndex);
		}
	}
	else
	{
		// Static mesh LOD
		const bool bValidMeshData = LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() > (uint32)VertexIndex;
		if (bValidMeshData)
		{
			OutColor = LODModel->VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex);
		}
	}
}

void FMeshPaintStaticMeshComponentAdapter::SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance /*= true*/)
{
	// Update the mesh!				
	if (bInstance)
	{
		if (!StaticMeshComponent.IsValid())
		{
			return;
		}

		FStaticMeshComponentLODInfo* InstanceMeshLODInfo = &StaticMeshComponent->LODData[MeshLODIndex];
		const bool bValidInstanceData = InstanceMeshLODInfo
			&& InstanceMeshLODInfo->OverrideVertexColors
			&& InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == LODModel->GetNumVertices();

		// If there is valid instance data update the color value
		if (bValidInstanceData)
		{
			check(InstanceMeshLODInfo->OverrideVertexColors);
			check((uint32)VertexIndex < InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices());
			check(InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == InstanceMeshLODInfo->PaintedVertices.Num());
			InstanceMeshLODInfo->OverrideVertexColors->VertexColor(VertexIndex) = Color;
			InstanceMeshLODInfo->PaintedVertices[VertexIndex].Color = Color;

#if WITH_EDITOR
			// If set on LOD level > 0 means we have per LOD painted vertex color data
			if (MeshLODIndex > 0)
			{
				StaticMeshComponent->bCustomOverrideVertexColorPerLOD = true;
			}
#endif
		}
	}	
	else
	{
		const bool bValidMeshData = LODModel->VertexBuffers.ColorVertexBuffer.GetNumVertices() >(uint32)VertexIndex;
		if (bValidMeshData)
		{
			LODModel->VertexBuffers.ColorVertexBuffer.VertexColor(VertexIndex) = Color;
		}
	}
}

FMatrix FMeshPaintStaticMeshComponentAdapter::GetComponentToWorldMatrix() const
{
	if (!StaticMeshComponent.IsValid())
	{
		return FMatrix::Identity;
	}

	return StaticMeshComponent->GetComponentToWorld().ToMatrixWithScale();
}

void FMeshPaintStaticMeshComponentAdapter::GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const
{
	OutTextureCoordinate = FVector2D(LODModel->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, ChannelIndex));
}


void FMeshPaintStaticMeshComponentAdapter::PreEdit()
{
	if (!StaticMeshComponent.IsValid())
	{
		return;
	}

	const bool bUsingInstancedVertexColors = true; // Currently we are only painting to instances 
	UStaticMesh* StaticMesh = ReferencedStaticMesh;
	if (bUsingInstancedVertexColors)
	{
		// Mark the mesh component as modified
		StaticMeshComponent->SetFlags(RF_Transactional);
#if WITH_EDITOR
		StaticMeshComponent->Modify();
		StaticMeshComponent->bCustomOverrideVertexColorPerLOD = (MeshLODIndex > 0);
#endif
		const int32 NumLODs = StaticMesh->GetNumLODs();
		const int32 MaxIndex = (MeshLODIndex == 0) ? NumLODs : (MeshLODIndex + 1);
		// Ensure LODData has enough entries in it, free not required.

		StaticMeshComponent->SetLODDataCount(NumLODs, NumLODs);

		// If LOD is 0, pre-edit all LODs. There's currently no way to tell from here
		// if VertexPaintSettings.bPaintOnSpecificLOD is set to true or not.
		for (int32 Index = MeshLODIndex; Index < MaxIndex; ++Index)
		{
			FStaticMeshComponentLODInfo& InstanceMeshLODInfo = StaticMeshComponent->LODData[Index];
			FStaticMeshLODResources& LODResource = StaticMesh->GetRenderData()->LODResources[Index];

			// Destroy the instance vertex  color array if it doesn't fit
			if (InstanceMeshLODInfo.OverrideVertexColors
				&& InstanceMeshLODInfo.OverrideVertexColors->GetNumVertices() != LODResource.GetNumVertices())
			{
				InstanceMeshLODInfo.ReleaseOverrideVertexColorsAndBlock();
			}

			if (InstanceMeshLODInfo.OverrideVertexColors)
			{
				// Destroy the cached paint data every paint. Painting redefines the source data.
				InstanceMeshLODInfo.PaintedVertices.Empty();
				InstanceMeshLODInfo.BeginReleaseOverrideVertexColors();
				FlushRenderingCommands();
			}
			else
			{
				// Setup the instance vertex color array if we don't have one yet
				InstanceMeshLODInfo.OverrideVertexColors = new FColorVertexBuffer;

				if ((int32)LODResource.VertexBuffers.ColorVertexBuffer.GetNumVertices() >= LODResource.GetNumVertices())
				{
					// copy mesh vertex colors to the instance ones
					InstanceMeshLODInfo.OverrideVertexColors->InitFromColorArray(&LODResource.VertexBuffers.ColorVertexBuffer.VertexColor(0), LODResource.GetNumVertices());
				}
				else
				{
					// Original mesh didn't have any colors, so just use a default color
					InstanceMeshLODInfo.OverrideVertexColors->InitFromSingleColor(FColor::White, LODResource.GetNumVertices());
				}

			}
		}
		// See if the component has to cache its mesh vertex positions associated with override colors
#if WITH_EDITOR
		StaticMeshComponent->CachePaintedDataIfNecessary();
		StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
#endif
	}
	else
	{
		// Dirty the mesh
		StaticMesh->SetFlags(RF_Transactional);
		StaticMesh->Modify();

		// Release the static mesh's resources.
		StaticMesh->ReleaseResources();

		// Flush the resource release commands to the rendering thread to ensure that the build doesn't occur while a resource is still
		// allocated, and potentially accessing the UStaticMesh.
		StaticMesh->ReleaseResourcesFence.Wait();
	}
}

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshesFactory

TSharedPtr<IMeshPaintComponentAdapter> FMeshPaintStaticMeshComponentAdapterFactory::Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InComponent))
	{
		if (StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			TSharedRef<FMeshPaintStaticMeshComponentAdapter> Result = MakeShareable(new FMeshPaintStaticMeshComponentAdapter());
			if (Result->Construct(InComponent, InMeshLODIndex))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
