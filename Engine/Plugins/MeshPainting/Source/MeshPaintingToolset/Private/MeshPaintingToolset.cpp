// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintingToolset.h"
#include "Modules/ModuleManager.h"



#define LOCTEXT_NAMESPACE "MeshPaintMode"

class FMeshPaintingToolsetModule : public IModuleInterface
{
public:
	FMeshPaintingToolsetModule()
	{
	}

	// FModuleInterface overrides
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

protected:


};


void FMeshPaintingToolsetModule::StartupModule()
{
}


void FMeshPaintingToolsetModule::ShutdownModule()
{
}




IMPLEMENT_MODULE(FMeshPaintingToolsetModule, MeshPaintingToolset)

#undef LOCTEXT_NAMESPACE
