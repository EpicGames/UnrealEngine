// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshSelect.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "InteractiveToolManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "ToolContextInterfaces.h"
#include "IMeshPaintComponentAdapter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSelect)


#define LOCTEXT_NAMESPACE "MeshSelection"
bool UVertexAdapterClickToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UVertexAdapterClickToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UVertexAdapterClickTool* NewTool = NewObject<UVertexAdapterClickTool>(SceneState.ToolManager);
	return NewTool;
}

bool UTextureColorAdapterClickToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UTextureColorAdapterClickToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UTextureColorAdapterClickTool* NewTool = NewObject<UTextureColorAdapterClickTool>(SceneState.ToolManager);
	return NewTool;
}

bool UTextureAssetAdapterClickToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UTextureAssetAdapterClickToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UTextureAssetAdapterClickTool* NewTool = NewObject<UTextureAssetAdapterClickTool>(SceneState.ToolManager);
	return NewTool;
}


UMeshClickTool::UMeshClickTool()
{
}

void UMeshClickTool::Setup()
{
	UInteractiveTool::Setup();

	// add default button input behaviors for devices
	USingleClickInputBehavior* MouseBehavior = NewObject<USingleClickInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->Modifiers.RegisterModifier(AdditiveSelectionModifier, FInputDeviceState::IsShiftKeyDown);
	AddInputBehavior(MouseBehavior);

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartMeshSelectTool", "Select a mesh. Switch tools to paint vertex colors, blend between textures, or paint directly onto a texture file."),
		EToolMessageLevel::UserNotification);

	// Set up selection mechanic to select valid meshes
	SelectionMechanic = NewObject<UMeshPaintSelectionMechanic>(this);
	SelectionMechanic->Setup(this);
}

void UMeshClickTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == AdditiveSelectionModifier)
	{
		SelectionMechanic->SetAddToSelectionSet(bIsOn);
	}
}

FInputRayHit UMeshClickTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	return SelectionMechanic->IsHitByClick(ClickPos);
}

void UMeshClickTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	SelectionMechanic->OnClicked(ClickPos);
}

bool UMeshClickTool::IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const
{
	return MeshAdapter.IsValid();
}

#undef LOCTEXT_NAMESPACE
