// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTexturePaintingTool.h"

#include "AssetRegistry/AssetData.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Components/MeshComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Editor/EditorEngine.h"
#include "Editor/TransBuffer.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IMeshPaintComponentAdapter.h"
#include "InteractiveToolManager.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "MeshPaintChannelSet.h"
#include "MeshPaintHelpers.h"
#include "MeshVertexPaintingTool.h"
#include "ObjectCacheContext.h"
#include "RenderingThread.h"
#include "RHIUtilities.h"
#include "ScopedTransaction.h"
#include "TextureCompiler.h"
#include "TexturePaintToolset.h"
#include "TextureResource.h"
#include "ToolDataVisualizer.h"
#include "VT/VirtualTextureAdapter.h"
#include "VT/VirtualTextureBuildSettings.h"
#include "UnrealEdGlobals.h"
#include "UObject/UObjectIterator.h"

extern UNREALED_API class UEditorEngine* GEditor;

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTexturePaintingTool)

#define LOCTEXT_NAMESPACE "MeshTextureBrush"

/*
 * ToolBuilder
 */

bool UMeshTextureColorPaintingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetSelectionSupportsTextureColorPaint();
}

UInteractiveTool* UMeshTextureColorPaintingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshTextureColorPaintingTool>(SceneState.ToolManager);
}

bool UMeshTextureAssetPaintingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetSelectionSupportsTextureAssetPaint();
}

UInteractiveTool* UMeshTextureAssetPaintingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshTextureAssetPaintingTool>(SceneState.ToolManager);
}


/*
 * Tool
 */

UMeshTexturePaintingTool::UMeshTexturePaintingTool()
{
	PropertyClass = UMeshTexturePaintingToolProperties::StaticClass();
}

void UMeshTexturePaintingTool::Setup()
{
	Super::Setup();

	bResultValid = false;
	bStampPending = false;

	FMeshPaintToolSettingHelpers::RestorePropertiesForClassHeirachy(this, BrushProperties);
	TextureProperties = Cast<UMeshTexturePaintingToolProperties>(BrushProperties);

	// Needed after restoring properties because the brush radius may be an output
	// property based on selection, so we shouldn't use the last stored value there.
	// We wouldn't have this problem if we restore properties before getting
	// BrushRelativeSizeRange, but that happens in the Super::Setup() call earlier.
	RecalculateBrushRadius();

	BrushStampIndicator->LineColor = FLinearColor::Green;

	SelectionMechanic = NewObject<UMeshPaintSelectionMechanic>(this);
	SelectionMechanic->Setup(this);

	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->Refresh();
	}
}

void UMeshTexturePaintingTool::Shutdown(EToolShutdownType ShutdownType)
{
	FinishPainting();

	{
		FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default|FMaterialUpdateContext::EOptions::SkipRecacheStaticPermutations);
		ClearAllTextureOverrides(MaterialUpdateContext);
	}

	PaintTargetData.Empty();

	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->Refresh();
	}

	FMeshPaintToolSettingHelpers::SavePropertiesForClassHeirachy(this, BrushProperties);

	Super::Shutdown(ShutdownType);
}

void UMeshTexturePaintingTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);
	FToolDataVisualizer Draw;
	Draw.BeginFrame(RenderAPI);
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem && LastBestHitResult.Component != nullptr)
	{
		BrushStampIndicator->bDrawIndicatorLines = true;
		static float WidgetLineThickness = 1.0f;
		static FLinearColor VertexPointColor = FLinearColor::White;
		static FLinearColor	HoverVertexPointColor = FLinearColor(0.3f, 1.0f, 0.3f);
		const float NormalLineSize(BrushProperties->BrushRadius * 0.35f);	// Make the normal line length a function of brush size
		static const FLinearColor NormalLineColor(0.3f, 1.0f, 0.3f);
		const FLinearColor BrushCueColor = (bArePainting ? FLinearColor(1.0f, 1.0f, 0.3f) : FLinearColor(0.3f, 1.0f, 0.3f));
 		const FLinearColor InnerBrushCueColor = (bArePainting ? FLinearColor(0.5f, 0.5f, 0.1f) : FLinearColor(0.1f, 0.5f, 0.1f));
		// Draw trace surface normal
		const FVector NormalLineEnd(LastBestHitResult.Location + LastBestHitResult.Normal * NormalLineSize);
		Draw.DrawLine(FVector(LastBestHitResult.Location), NormalLineEnd, NormalLineColor, WidgetLineThickness);

		for (UMeshComponent* CurrentComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(Cast<UMeshComponent>(CurrentComponent));
			if (IsMeshAdapterSupported(MeshAdapter))
			{
				const FMatrix ComponentToWorldMatrix = MeshAdapter->GetComponentToWorldMatrix();
				FViewCameraState CameraState;
				GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
				const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(CameraState.Position));
				const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(LastBestHitResult.Location));

				// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
				const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushProperties->BrushRadius, 0.0f, 0.0f)).Size();
				const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;
			}
		}
	}
	else
	{
		BrushStampIndicator->bDrawIndicatorLines = false;
	}
	Draw.EndFrame();
	UpdateResult();
}

void UMeshTexturePaintingTool::OnTick(float DeltaTime)
{
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		TArray<UMeshComponent*> SelectedMeshComponents = MeshPaintingSubsystem->GetSelectedMeshComponents();

		if (bRequestPaintBucketFill)
		{
			FMeshPaintParameters BucketFillParams;

			// NOTE: We square the brush strength to maximize slider precision in the low range
			const float BrushStrength = BrushProperties->BrushStrength * BrushProperties->BrushStrength;

			// Mesh paint settings; Only fill out relevant parameters
			{
				BucketFillParams.PaintAction = EMeshPaintModeAction::Paint;
				BucketFillParams.BrushColor = TextureProperties->PaintColor;
				BucketFillParams.BrushStrength = BrushStrength;
				TextureProperties->GetWriteMask(BucketFillParams.bWriteRed, BucketFillParams.bWriteGreen, BucketFillParams.bWriteBlue, BucketFillParams.bWriteAlpha);
				BucketFillParams.bUseFillBucket = true;
			}

			for (int32 j = 0; j < SelectedMeshComponents.Num(); ++j)
			{
				UMeshComponent* SelectedComponent = SelectedMeshComponents[j];
				TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(SelectedComponent);
				if (!IsMeshAdapterSupported(MeshAdapter))
				{
					continue;
				}

				const int32 UVChannel = GetSelectedUVChannel(SelectedComponent);
				if (UVChannel < 0 || UVChannel >= MeshAdapter->GetNumUVChannels())
				{
					continue;
				}

				TArray<UTexture const*> Textures;
				const UTexture2D* TargetTexture2D = GetSelectedPaintTexture(SelectedComponent);
				if (TargetTexture2D == nullptr)
				{
					continue;
				}
					
				Textures.Add(TargetTexture2D);

				FPaintTexture2DData* TextureData = GetPaintTargetData(TargetTexture2D);
				if (TextureData)
				{
					Textures.Add(TextureData->PaintRenderTargetTexture);
				}

				TArray<FTexturePaintMeshSectionInfo> MaterialSections;
				UTexturePaintToolset::RetrieveMeshSectionsForTextures(SelectedComponent, 0 /*CachedLODIndex*/, Textures, MaterialSections);

				TArray<FTexturePaintTriangleInfo> TrianglePaintInfoArray;
				FPerTrianglePaintAction TempAction = FPerTrianglePaintAction::CreateUObject(this, &UMeshTexturePaintingTool::GatherTextureTriangles, &TrianglePaintInfoArray, &MaterialSections, UVChannel);

				// We are flooding the texture, so all triangles are influenced

				const TArray<uint32>& MeshIndices = MeshAdapter->GetMeshIndices();
				int32 TriangleIndices[3];

				for (int32 i = 0; i < MeshIndices.Num(); i += 3)
				{
					TriangleIndices[0] = MeshIndices[i + 0];
					TriangleIndices[1] = MeshIndices[i + 1];
					TriangleIndices[2] = MeshIndices[i + 2];
					TempAction.Execute(MeshAdapter.Get(), i / 3, TriangleIndices);
				}

				ApplyWorldChannelUVs(TrianglePaintInfoArray);

				// Painting textures
				UTexture2D* SelectedPaintTexure = GetSelectedPaintTexture(SelectedComponent);
				if (PaintingTexture2D != nullptr && PaintingTexture2D != SelectedPaintTexure)
				{
					// Texture has changed, so finish up with our previous texture
					FinishPaintingTexture();
				}

				if (PaintingTexture2D == nullptr)
				{
					StartPaintingTexture(SelectedComponent, *MeshAdapter);
				}

				FMeshPaintParameters* LastParams = nullptr;
				PaintTexture(BucketFillParams, UVChannel, TrianglePaintInfoArray, SelectedComponent, *MeshAdapter, LastParams);
			}
		}

		UMeshComponent* FirstSelectedComponent = SelectedMeshComponents.IsValidIndex(0) ? SelectedMeshComponents[0] : nullptr;
		if (MeshPaintingSubsystem->bNeedsRecache || (PaintableTextures.Num() > 0 && GetSelectedPaintTexture(FirstSelectedComponent) == nullptr))
		{
			FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default|FMaterialUpdateContext::EOptions::SkipRecacheStaticPermutations);

			ClearAllTextureOverrides(MaterialUpdateContext);

			CacheSelectionData();
			CacheTexturePaintData();

			SetAllTextureOverrides(MaterialUpdateContext);
		}
	}

	if (bStampPending)
	{
		Paint(PendingStampRay.Origin, PendingStampRay.Direction);
		bStampPending = false;

		// flow
		if (bInDrag && TextureProperties && TextureProperties->bEnableFlow)
		{
			bStampPending = true;
		}
	}

	// Wait till end of the tick to finish painting so all systems in-between know if we've painted this frame
	if (bRequestPaintBucketFill)
	{
		if (PaintingTexture2D != nullptr)
		{
			FinishPaintingTexture();
			FinishPainting();
		}

		bRequestPaintBucketFill = false;
	}
}

void UMeshTexturePaintingTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);
	bResultValid = false;
}


double UMeshTexturePaintingTool::EstimateMaximumTargetDimension()
{
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		FBoxSphereBounds::Builder ExtentsBuilder;
		for (UMeshComponent* SelectedComponent : MeshPaintingSubsystem->GetSelectedMeshComponents())
		{
			ExtentsBuilder += SelectedComponent->Bounds;
		}

		if (ExtentsBuilder.IsValid())
		{
			return FBoxSphereBounds(ExtentsBuilder).BoxExtent.GetAbsMax();
		}
	}

	return Super::EstimateMaximumTargetDimension();
}

double UMeshTexturePaintingTool::CalculateTargetEdgeLength(int TargetTriCount)
{
	double TargetTriArea = InitialMeshArea / (double)TargetTriCount;
	double EdgeLen = (TargetTriArea);
	return (double)FMath::RoundToInt(EdgeLen*100.0) / 100.0;
}

bool UMeshTexturePaintingTool::Paint(const FVector& InRayOrigin, const FVector& InRayDirection)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintModeAction PaintAction = GetShiftToggle() ? EMeshPaintModeAction::Erase : EMeshPaintModeAction::Paint;
	const float PaintStrength = 1.0f; //Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	TPair<FVector, FVector> Ray(InRayOrigin, InRayDirection);
	return PaintInternal(MakeArrayView(&Ray, 1), PaintAction, PaintStrength);
}

bool UMeshTexturePaintingTool::Paint(const TArrayView<TPair<FVector, FVector>>& Rays)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintModeAction PaintAction = GetShiftToggle() ? EMeshPaintModeAction::Erase : EMeshPaintModeAction::Paint;

	const float PaintStrength = 1.0f; //Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	return PaintInternal(Rays, PaintAction, PaintStrength);
}

void UMeshTexturePaintingTool::CacheSelectionData()
{
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		MeshPaintingSubsystem->ClearPaintableMeshComponents();

		//Determine LOD level to use for painting(can only paint on LODs in vertex mode)
		const int32 PaintLODIndex = 0;
		//Determine UV channel to use while painting textures
		const int32 UVChannel = 0;

		MeshPaintingSubsystem->CacheSelectionData(PaintLODIndex, UVChannel);
	}
}

bool UMeshTexturePaintingTool::PaintInternal(const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintModeAction PaintAction, float PaintStrength)
{
	TArray<FPaintRayResults> PaintRayResults;
	PaintRayResults.AddDefaulted(Rays.Num());

	bool bAnyPaintApplied = false;

	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		TMap<UMeshComponent*, TArray<int32>> HoveredComponents;

		const float BrushRadius = BrushProperties->BrushRadius;
		const bool bIsPainting = (PaintAction == EMeshPaintModeAction::Paint);
		const float InStrengthScale = PaintStrength;;

		// Fire out a ray to see if there is a *selected* component under the mouse cursor that can be painted.
		for (int32 i = 0; i < Rays.Num(); ++i)
		{
			const FVector& RayOrigin = Rays[i].Key;
			const FVector& RayDirection = Rays[i].Value;
			FHitResult& BestTraceResult = PaintRayResults[i].BestTraceResult;

			const FVector TraceStart(RayOrigin);
			const FVector TraceEnd(RayOrigin + RayDirection * HALF_WORLD_MAX);

			for (UMeshComponent* MeshComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
			{
				TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);

				// Ray trace
				FHitResult TraceHitResult(1.0f);

				if (MeshAdapter->LineTraceComponent(TraceHitResult, TraceStart, TraceEnd, FCollisionQueryParams(SCENE_QUERY_STAT(Paint), true)))
				{
					// Find the closest impact
					if ((BestTraceResult.GetComponent() == nullptr) || (TraceHitResult.Time < BestTraceResult.Time))
					{
						BestTraceResult = TraceHitResult;
					}
				}
			}

			UMeshComponent* BestTraceMeshComponent = Cast<UMeshComponent>(BestTraceResult.GetComponent());
			// If painting texture assets, just use the BestTraceMeshComponent as we only support painting a single mesh at a time in that mode.
			const bool bAllowMultiselect = AllowsMultiselect();

			bool bUsed = false;
			for (UMeshComponent* MeshComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
			{
				if (MeshComponent == BestTraceMeshComponent)
				{
					HoveredComponents.FindOrAdd(MeshComponent).Add(i);
					bUsed = true;
				}
				else if (bAllowMultiselect)
				{
					FSphere Sphere(BestTraceResult.Location, BrushRadius);
					if (MeshComponent->GetLocalBounds().GetSphere().TransformBy(MeshComponent->GetComponentTransform()).Intersects(Sphere))
					{
						HoveredComponents.FindOrAdd(MeshComponent).Add(i);
						bUsed = true;
					}
				}
			}

			if (bUsed)
			{
				FVector BrushXAxis, BrushYAxis;
				BestTraceResult.Normal.FindBestAxisVectors(BrushXAxis, BrushYAxis);
				// Display settings
				const float VisualBiasDistance = 0.15f;
				const FVector BrushVisualPosition = BestTraceResult.Location + BestTraceResult.Normal * VisualBiasDistance;

				const FLinearColor PaintColor = TextureProperties->PaintColor;
				const FLinearColor EraseColor = TextureProperties->EraseColor;

				// NOTE: We square the brush strength to maximize slider precision in the low range
				const float BrushStrength = BrushProperties->BrushStrength * BrushProperties->BrushStrength * InStrengthScale;

				const float BrushDepth = BrushRadius;

				// Mesh paint settings
				FMeshPaintParameters& Params = PaintRayResults[i].Params;
				{
					Params.PaintAction = PaintAction;
					Params.BrushPosition = BestTraceResult.Location;
					Params.BrushNormal = BestTraceResult.Normal;
					Params.BrushColor = bIsPainting ? PaintColor : EraseColor;
					Params.SquaredBrushRadius = BrushRadius * BrushRadius;
					Params.BrushRadialFalloffRange = BrushProperties->BrushFalloffAmount * BrushRadius;
					Params.InnerBrushRadius = BrushRadius - Params.BrushRadialFalloffRange;
					Params.BrushDepth = BrushDepth;
					Params.BrushDepthFalloffRange = BrushProperties->BrushFalloffAmount * BrushDepth;
					Params.InnerBrushDepth = BrushDepth - Params.BrushDepthFalloffRange;
					Params.BrushStrength = BrushStrength;
					Params.BrushToWorldMatrix = FMatrix(BrushXAxis, BrushYAxis, Params.BrushNormal, Params.BrushPosition);
					Params.InverseBrushToWorldMatrix = Params.BrushToWorldMatrix.Inverse();
					TextureProperties->GetWriteMask(Params.bWriteRed, Params.bWriteGreen, Params.bWriteBlue, Params.bWriteAlpha);
					FVector BrushSpaceVertexPosition = Params.InverseBrushToWorldMatrix.TransformVector(FVector4(Params.BrushPosition, 1.0f));
					Params.BrushPosition2D = FVector2f(BrushSpaceVertexPosition.X, BrushSpaceVertexPosition.Y);
				}
			}
		}

		if (HoveredComponents.Num() > 0)
		{
			if (bArePainting == false)
			{
				bArePainting = true;
			}

			// Iterate over the selected meshes under the cursor and paint them!
			for (auto& Entry : HoveredComponents)
			{
				UMeshComponent* HoveredComponent = Entry.Key;
				TArray<int32>& PaintRayResultIds = Entry.Value;
			
				TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(HoveredComponent);
				if (!IsMeshAdapterSupported(MeshAdapter))
				{
					continue;
				}
				
				const int32 UVChannel = GetSelectedUVChannel(HoveredComponent);
				if (UVChannel < 0 || UVChannel >= MeshAdapter->GetNumUVChannels())
				{
					continue;
				}

				TArray<UTexture const*> Textures;
				const UTexture2D* TargetTexture2D = GetSelectedPaintTexture(HoveredComponent);
				if (TargetTexture2D == nullptr)
				{
					continue;
				}
					
				Textures.Add(TargetTexture2D);

				FPaintTexture2DData* TextureData = GetPaintTargetData(TargetTexture2D);
				if (TextureData)
				{
					Textures.Add(TextureData->PaintRenderTargetTexture);
				}

				TArray<FTexturePaintMeshSectionInfo> MaterialSections;
				UTexturePaintToolset::RetrieveMeshSectionsForTextures(HoveredComponent, 0/*CachedLODIndex*/, Textures, MaterialSections);

				bool bPaintApplied = false;
				TArray<FTexturePaintTriangleInfo> TrianglePaintInfoArray;
				if (PaintRayResultIds.Num() > 0)
				{
					const int32 PaintRayResultId = PaintRayResultIds[0];
					const FVector& BestTraceResultLocation = PaintRayResults[PaintRayResultId].BestTraceResult.Location;
					FViewCameraState CameraState;
					GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
					bPaintApplied |= MeshPaintingSubsystem->ApplyPerTrianglePaintAction(MeshAdapter.Get(), CameraState.Position, BestTraceResultLocation, BrushProperties, FPerTrianglePaintAction::CreateUObject(this, &UMeshTexturePaintingTool::GatherTextureTriangles, &TrianglePaintInfoArray, &MaterialSections, UVChannel), TextureProperties->bOnlyFrontFacingTriangles);
				}

				if (!bPaintApplied)
				{
					continue;
				}

				ApplyWorldChannelUVs(TrianglePaintInfoArray);

				// Painting textures
				bAnyPaintApplied = true;

				UTexture2D* SelectedPaintTexure = GetSelectedPaintTexture(HoveredComponent);
				if (PaintingTexture2D != nullptr && PaintingTexture2D != SelectedPaintTexure)
				{
					// Texture has changed, so finish up with our previous texture
					FinishPaintingTexture();
				}

				if (PaintingTexture2D == nullptr)
				{
					StartPaintingTexture(HoveredComponent, *MeshAdapter);
				}

				if (PaintingTexture2D != nullptr)
				{
					if (PaintRayResultIds.Num() > 0)
					{
						const int32 PaintRayResultId = PaintRayResultIds[0];
						FMeshPaintParameters& Params = PaintRayResults[PaintRayResultId].Params;
						FMeshPaintParameters* LastParams = nullptr;
						if (LastPaintRayResults.Num() > PaintRayResultId)
						{
							LastParams = &LastPaintRayResults[PaintRayResultId].Params;
						}

						PaintTexture(Params, UVChannel, TrianglePaintInfoArray, HoveredComponent, *MeshAdapter, LastParams);
					}
				}
			}
		}
	}

	LastPaintRayResults = MoveTemp(PaintRayResults);
	return bAnyPaintApplied;
}

/** Painting texture to use in material override should be the virtual texture adapter if it exists. */
static UTexture* GetTextureForMaterialOverride(FPaintTexture2DData const& TextureData)
{
	UTexture* RenderTarget = TextureData.PaintRenderTargetTexture;
	UTexture* RenderTargetAdapter = TextureData.PaintRenderTargetTextureAdapter;
	return RenderTargetAdapter != nullptr ? RenderTargetAdapter : RenderTarget;
}

void UMeshTexturePaintingTool::AddTextureOverrideToComponent(FPaintTexture2DData& TextureData, UMeshComponent* MeshComponent, const IMeshPaintComponentAdapter* MeshPaintAdapter, FMaterialUpdateContext& MaterialUpdateContext)
{
	if (MeshComponent && MeshPaintAdapter)
	{
		if (!TextureData.TextureOverrideComponents.Contains(MeshComponent))
		{
			TextureData.TextureOverrideComponents.AddUnique(MeshComponent);

			MeshPaintAdapter->ApplyOrRemoveTextureOverride(TextureData.PaintingTexture2D, GetTextureForMaterialOverride(TextureData), MaterialUpdateContext);
		}
	}
}

void UMeshTexturePaintingTool::UpdateResult()
{
	GetToolManager()->PostInvalidation();

	bResultValid = true;
}

FInputRayHit UMeshTexturePaintingTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	bCachedClickRay = false;
	if (!HitTest(PressPos.WorldRay, OutHit))
	{
		UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
		const bool bFallbackClick = MeshPaintingSubsystem->GetSelectedMeshComponents().Num() > 0;
		if (SelectionMechanic->IsHitByClick(PressPos, bFallbackClick).bHit)
		{
			bCachedClickRay = true;
			PendingClickRay = PressPos.WorldRay;
			PendingClickScreenPosition = PressPos.ScreenPosition;
			return FInputRayHit(0.0);
		}
	}

	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem && LastBestHitResult.Component != nullptr && MeshPaintingSubsystem->LastPaintedComponent != LastBestHitResult.Component)
	{
		MeshPaintingSubsystem->LastPaintedComponent = (UMeshComponent*)LastBestHitResult.Component.Get();
	}

	return Super::CanBeginClickDragSequence(PressPos);
}

void UMeshTexturePaintingTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	Super::OnUpdateModifierState(ModifierID, bIsOn);
	SelectionMechanic->SetAddToSelectionSet(bShiftToggle);
}

void UMeshTexturePaintingTool::OnBeginDrag(const FRay& Ray)
{
	Super::OnBeginDrag(Ray);
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		bInDrag = true;

		// apply initial stamp
		PendingStampRay = Ray;
		bStampPending = true;
	}
	else if (bCachedClickRay)
	{
		FInputDeviceRay InputDeviceRay = FInputDeviceRay(PendingClickRay, PendingClickScreenPosition);
		SelectionMechanic->SetAddToSelectionSet(bShiftToggle);
		SelectionMechanic->OnClicked(InputDeviceRay);
		bCachedClickRay = false;
		RecalculateBrushRadius();
	}
}

void UMeshTexturePaintingTool::OnUpdateDrag(const FRay& Ray)
{
	Super::OnUpdateDrag(Ray);
	if (bInDrag)
	{
		PendingStampRay = Ray;
		bStampPending = true;
	}
}

void UMeshTexturePaintingTool::OnEndDrag(const FRay& Ray)
{
	FinishPaintingTexture();
	FinishPainting();
	bStampPending = false;
	bInDrag = false;
}

bool UMeshTexturePaintingTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	bool bUsed = false;
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->FindHitResult(Ray, OutHit);
		LastBestHitResult = OutHit;
		bUsed = OutHit.bBlockingHit;
	}
	return bUsed;
}

void UMeshTexturePaintingTool::FinishPainting()
{
	PaintingTransaction.Reset();
	bArePainting = false;
	bRequiresRuntimeVirtualTextureUpdates = false;
}

FPaintTexture2DData* UMeshTexturePaintingTool::GetPaintTargetData(const UTexture2D* InTexture)
{
	return PaintTargetData.Find(InTexture);
}

FPaintTexture2DData* UMeshTexturePaintingTool::AddPaintTargetData(UTexture2D* InTexture)
{
	checkf(InTexture != nullptr, TEXT("Invalid Texture ptr"));

	/** Only create new target if we haven't gotten one already  */
	FPaintTexture2DData* TextureData = GetPaintTargetData(InTexture);
	if (TextureData == nullptr)
	{
		// If we didn't find data associated with this texture we create a new entry and return a reference to it.
		//   Note: This reference is only valid until the next change to any key in the map.
		TextureData = &PaintTargetData.Add(InTexture, FPaintTexture2DData(InTexture));
	}
	return TextureData;
}

void UMeshTexturePaintingTool::GatherTextureTriangles(IMeshPaintComponentAdapter* Adapter, int32 TriangleIndex, const int32 VertexIndices[3], TArray<FTexturePaintTriangleInfo>* TriangleInfo, TArray<FTexturePaintMeshSectionInfo>* SectionInfos, int32 UVChannelIndex)
{
	/** Retrieve triangles eligible for texture painting */
	bool bAdd = SectionInfos->Num() == 0;
	for (const FTexturePaintMeshSectionInfo& SectionInfo : *SectionInfos)
	{
		if (TriangleIndex >= SectionInfo.FirstIndex && TriangleIndex < SectionInfo.LastIndex)
		{
			bAdd = true;
			break;
		}
	}

	if (bAdd)
	{
		FTexturePaintTriangleInfo Info;
		Adapter->GetVertexPosition(VertexIndices[0], Info.TriVertices[0]);
		Adapter->GetVertexPosition(VertexIndices[1], Info.TriVertices[1]);
		Adapter->GetVertexPosition(VertexIndices[2], Info.TriVertices[2]);
		Info.TriVertices[0] = Adapter->GetComponentToWorldMatrix().TransformPosition(Info.TriVertices[0]);
		Info.TriVertices[1] = Adapter->GetComponentToWorldMatrix().TransformPosition(Info.TriVertices[1]);
		Info.TriVertices[2] = Adapter->GetComponentToWorldMatrix().TransformPosition(Info.TriVertices[2]);
		Adapter->GetTextureCoordinate(VertexIndices[0], UVChannelIndex, Info.TriUVs[0]);
		Adapter->GetTextureCoordinate(VertexIndices[1], UVChannelIndex, Info.TriUVs[1]);
		Adapter->GetTextureCoordinate(VertexIndices[2], UVChannelIndex, Info.TriUVs[2]);
		TriangleInfo->Add(Info);
	}
}

void UMeshTexturePaintingTool::StartPaintingTexture(UMeshComponent* InMeshComponent, const IMeshPaintComponentAdapter& GeometryInfo)
{
	check(InMeshComponent != nullptr);
	check(PaintingTexture2D == nullptr);

	// Only start new transaction if not in one currently
	if (!PaintingTransaction.IsValid())
	{
		PaintingTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("MeshPaintMode_TexturePaint_Transaction", "Texture Paint"));
	}

	const auto FeatureLevel = InMeshComponent->GetWorld()->GetFeatureLevel();

	UTexture2D* Texture2D = GetSelectedPaintTexture(InMeshComponent);
	if (Texture2D == nullptr)
	{
		return;
	}

	bool bStartedPainting = false;
	FPaintTexture2DData* TextureData = GetPaintTargetData(Texture2D);

	// Check all the materials on the mesh to see if the user texture is there
	int32 MaterialIndex = 0;
	UMaterialInterface* MaterialToCheck = InMeshComponent->GetMaterial(MaterialIndex);

	Texture2D->BlockOnAnyAsyncBuild();
	bool bIsSourceTextureStreamedIn = Texture2D->IsFullyStreamedIn() && !Texture2D->HasPendingInitOrStreaming();

	// IMeshPaintComponentAdapter::DefaultQueryPaintableTextures already filters out un-used textures
	if (!bIsSourceTextureStreamedIn)
	{
		Texture2D->SetForceMipLevelsToBeResident(30.0f);
		Texture2D->bForceMiplevelsToBeResident = true;
		Texture2D->WaitForStreaming();
		bIsSourceTextureStreamedIn = Texture2D->IsFullyStreamedIn() && !Texture2D->HasPendingInitOrStreaming();
	}

	while (MaterialToCheck != nullptr)
	{
		if (!bStartedPainting)
		{
			const int32 TextureWidth = Texture2D->Source.GetSizeX();
			const int32 TextureHeight = Texture2D->Source.GetSizeY();

			check(TextureData != nullptr);

			const int32 BrushTargetTextureWidth = TextureWidth;
			const int32 BrushTargetTextureHeight = TextureHeight;

			// Create the rendertarget used to store our paint delta
			if (TextureData->BrushRenderTargetTexture == nullptr ||
				TextureData->BrushRenderTargetTexture->GetSurfaceWidth() != BrushTargetTextureWidth ||
				TextureData->BrushRenderTargetTexture->GetSurfaceHeight() != BrushTargetTextureHeight)
			{
				TextureData->BrushRenderTargetTexture = nullptr;
				TextureData->BrushRenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
				const bool bForceLinearGamma = true;
				TextureData->BrushRenderTargetTexture->ClearColor = FLinearColor::Black;
				TextureData->BrushRenderTargetTexture->bNeedsTwoCopies = true;
				TextureData->BrushRenderTargetTexture->InitCustomFormat(BrushTargetTextureWidth, BrushTargetTextureHeight, PF_A16B16G16R16, bForceLinearGamma);
				TextureData->BrushRenderTargetTexture->UpdateResourceImmediate();
				TextureData->BrushRenderTargetTexture->AddressX = TextureData->PaintRenderTargetTexture->AddressX;
				TextureData->BrushRenderTargetTexture->AddressY = TextureData->PaintRenderTargetTexture->AddressY;
			}

			if (TextureProperties->bEnableSeamPainting)
			{
				// Create the rendertarget used to store a mask for our paint delta area 
				if (TextureData->BrushMaskRenderTargetTexture == nullptr ||
					TextureData->BrushMaskRenderTargetTexture->GetSurfaceWidth() != BrushTargetTextureWidth ||
					TextureData->BrushMaskRenderTargetTexture->GetSurfaceHeight() != BrushTargetTextureHeight)
				{
					TextureData->BrushMaskRenderTargetTexture = nullptr;
					TextureData->BrushMaskRenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
					const bool bForceLinearGamma = true;
					TextureData->BrushMaskRenderTargetTexture->ClearColor = FLinearColor::Black;
					TextureData->BrushMaskRenderTargetTexture->bNeedsTwoCopies = true;
					TextureData->BrushMaskRenderTargetTexture->InitCustomFormat(BrushTargetTextureWidth, BrushTargetTextureHeight, PF_G8, bForceLinearGamma);
					TextureData->BrushMaskRenderTargetTexture->UpdateResourceImmediate();
					TextureData->BrushMaskRenderTargetTexture->AddressX = TextureData->PaintRenderTargetTexture->AddressX;
					TextureData->BrushMaskRenderTargetTexture->AddressY = TextureData->PaintRenderTargetTexture->AddressY;
				}

				// Create the rendertarget used to store a texture seam mask
				if (TextureData->SeamMaskRenderTargetTexture == nullptr ||
					TextureData->SeamMaskRenderTargetTexture->GetSurfaceWidth() != TextureWidth ||
					TextureData->SeamMaskRenderTargetTexture->GetSurfaceHeight() != TextureHeight)
				{
					TextureData->SeamMaskRenderTargetTexture = nullptr;
					TextureData->SeamMaskRenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
					const bool bForceLinearGamma = true;
					TextureData->SeamMaskRenderTargetTexture->ClearColor = FLinearColor::Black;
					TextureData->SeamMaskRenderTargetTexture->bNeedsTwoCopies = true;
					TextureData->SeamMaskRenderTargetTexture->InitCustomFormat(BrushTargetTextureWidth, BrushTargetTextureHeight, PF_G8, bForceLinearGamma);
					TextureData->SeamMaskRenderTargetTexture->UpdateResourceImmediate();
					TextureData->SeamMaskRenderTargetTexture->AddressX = TextureData->PaintRenderTargetTexture->AddressX;
					TextureData->SeamMaskRenderTargetTexture->AddressY = TextureData->PaintRenderTargetTexture->AddressY;
					TextureData->SeamMaskComponent = nullptr;
				}
			}

			bStartedPainting = true;
			UTexture2D* Texture2DPaintBrush = TextureProperties->PaintBrush;
			if (Texture2DPaintBrush)
			{
				const int32 PaintBrushTextureWidth = Texture2DPaintBrush->Source.GetSizeX();
				const int32 PaintBrushTextureHeight = Texture2DPaintBrush->Source.GetSizeY();
				if (TextureData->PaintBrushRenderTargetTexture == nullptr ||
					TextureData->PaintBrushRenderTargetTexture->GetSurfaceWidth() != PaintBrushTextureWidth ||
					TextureData->PaintBrushRenderTargetTexture->GetSurfaceHeight() != PaintBrushTextureHeight)
				{
					TextureData->PaintBrushRenderTargetTexture = nullptr;
					TextureData->PaintBrushRenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
					TextureData->PaintBrushRenderTargetTexture->bNeedsTwoCopies = true;
					const bool bForceLinearGamma = true;
					TextureData->PaintBrushRenderTargetTexture->ClearColor = FLinearColor::Black;
					TextureData->PaintBrushRenderTargetTexture->InitCustomFormat(PaintBrushTextureWidth, PaintBrushTextureHeight, PF_A16B16G16R16, bForceLinearGamma);
					TextureData->PaintBrushRenderTargetTexture->UpdateResourceImmediate();
				}
				TextureData->PaintBrushRenderTargetTexture->AddressX = Texture2DPaintBrush->AddressX;
				TextureData->PaintBrushRenderTargetTexture->AddressY = Texture2DPaintBrush->AddressY;
			}
			else
			{
				TextureData->PaintBrushRenderTargetTexture = nullptr;
			}

			check(Texture2D != nullptr);
			PaintingTexture2D = Texture2D;
		}

		MaterialIndex++;
		MaterialToCheck = InMeshComponent->GetMaterial(MaterialIndex);
	}

	if (bIsSourceTextureStreamedIn && bStartedPainting)
	{
		check(Texture2D != nullptr);
		PaintingTexture2D = Texture2D;

		if (TextureProperties->PaintBrush != nullptr && TextureData->PaintBrushRenderTargetTexture != nullptr)
		{
			UTexturePaintToolset::SetupInitialRenderTargetData(TextureProperties->PaintBrush, TextureData->PaintBrushRenderTargetTexture);
		}
	}

	{
		// Set flag for whether to update runtime virtual textures while painting.
		bRequiresRuntimeVirtualTextureUpdates = false;
		FObjectCacheContextScope ObjectCacheScope;
		for (UMaterialInterface* Material : ObjectCacheScope.GetContext().GetMaterialsAffectedByTexture(Texture2D))
		{
			bRequiresRuntimeVirtualTextureUpdates |= Material->WritesToRuntimeVirtualTexture();
		}
	}
}

void UMeshTexturePaintingTool::PaintTexture(FMeshPaintParameters& InParams, int32 UVChannel, TArray<FTexturePaintTriangleInfo>& InInfluencedTriangles, UMeshComponent* MeshComponent, const IMeshPaintComponentAdapter& GeometryInfo, FMeshPaintParameters* LastParams)
{
	// We bail early if there are no influenced triangles
	if (InInfluencedTriangles.Num() <= 0)
	{
		return;
	}

	check(GEditor && GEditor->GetEditorWorldContext().World());
	const auto FeatureLevel = GEditor->GetEditorWorldContext().World()->GetFeatureLevel();

	FPaintTexture2DData* TextureData = GetPaintTargetData(PaintingTexture2D);
	check(TextureData != nullptr && TextureData->PaintRenderTargetTexture != nullptr);

	// Copy the current image to the brush rendertarget texture.
	{
		check(TextureData->BrushRenderTargetTexture != nullptr);
		UTexturePaintToolset::CopyTextureToRenderTargetTexture(TextureData->PaintRenderTargetTexture, TextureData->BrushRenderTargetTexture, FeatureLevel);
	}

	const bool bEnableSeamPainting = TextureProperties->bEnableSeamPainting;
	const FMatrix WorldToBrushMatrix = InParams.InverseBrushToWorldMatrix;

	// Grab the actual render target resource from the textures.  Note that we're absolutely NOT ALLOWED to
	// dereference these pointers.  We're just passing them along to other functions that will use them on the render
	// thread.  The only thing we're allowed to do is check to see if they are nullptr or not.
	FTextureRenderTargetResource* BrushRenderTargetResource = TextureData->BrushRenderTargetTexture->GameThread_GetRenderTargetResource();
	check(BrushRenderTargetResource != nullptr);

	// Create a canvas for the brush render target.
	FCanvas BrushPaintCanvas(BrushRenderTargetResource, nullptr, FGameTime(), FeatureLevel);

	// Parameters for brush paint
	TRefCountPtr< FMeshPaintBatchedElementParameters > MeshPaintBatchedElementParameters(new FMeshPaintBatchedElementParameters());
	{
		MeshPaintBatchedElementParameters->ShaderParams.PaintBrushTexture = TextureData->PaintBrushRenderTargetTexture;
		if (LastParams)
		{
			MeshPaintBatchedElementParameters->ShaderParams.PaintBrushDirectionVector = InParams.BrushPosition2D - LastParams->BrushPosition2D;
			MeshPaintBatchedElementParameters->ShaderParams.bRotateBrushTowardsDirection = TextureProperties->bRotateBrushTowardsDirection;
		}
		else
		{
			MeshPaintBatchedElementParameters->ShaderParams.PaintBrushDirectionVector = FVector2f(0.0f, 0.0f);
			MeshPaintBatchedElementParameters->ShaderParams.bRotateBrushTowardsDirection = false;
		}
		MeshPaintBatchedElementParameters->ShaderParams.PaintBrushRotationOffset = TextureProperties->PaintBrushRotationOffset;
		MeshPaintBatchedElementParameters->ShaderParams.bUseFillBucket = InParams.bUseFillBucket;
		MeshPaintBatchedElementParameters->ShaderParams.CloneTexture = TextureData->BrushRenderTargetTexture;
		MeshPaintBatchedElementParameters->ShaderParams.WorldToBrushMatrix = WorldToBrushMatrix;
		MeshPaintBatchedElementParameters->ShaderParams.BrushRadius = InParams.InnerBrushRadius + InParams.BrushRadialFalloffRange;
		MeshPaintBatchedElementParameters->ShaderParams.BrushRadialFalloffRange = InParams.BrushRadialFalloffRange;
		MeshPaintBatchedElementParameters->ShaderParams.BrushDepth = InParams.InnerBrushDepth + InParams.BrushDepthFalloffRange;
		MeshPaintBatchedElementParameters->ShaderParams.BrushDepthFalloffRange = InParams.BrushDepthFalloffRange;
		MeshPaintBatchedElementParameters->ShaderParams.BrushStrength = InParams.BrushStrength;
		MeshPaintBatchedElementParameters->ShaderParams.BrushColor = InParams.BrushColor;
		MeshPaintBatchedElementParameters->ShaderParams.RedChannelFlag = InParams.bWriteRed;
		MeshPaintBatchedElementParameters->ShaderParams.GreenChannelFlag = InParams.bWriteGreen;
		MeshPaintBatchedElementParameters->ShaderParams.BlueChannelFlag = InParams.bWriteBlue;
		MeshPaintBatchedElementParameters->ShaderParams.AlphaChannelFlag = InParams.bWriteAlpha;
		MeshPaintBatchedElementParameters->ShaderParams.GenerateMaskFlag = false;
	}

	FBatchedElements* BrushPaintBatchedElements = BrushPaintCanvas.GetBatchedElements(FCanvas::ET_Triangle, MeshPaintBatchedElementParameters, nullptr, SE_BLEND_Opaque);
	BrushPaintBatchedElements->AddReserveVertices(InInfluencedTriangles.Num() * 3);
	BrushPaintBatchedElements->AddReserveTriangles(InInfluencedTriangles.Num(), nullptr, SE_BLEND_Opaque);

	FHitProxyId BrushPaintHitProxyId = BrushPaintCanvas.GetHitProxyId();

	TSharedPtr<FCanvas> BrushMaskCanvas;
	TRefCountPtr< FMeshPaintBatchedElementParameters > MeshPaintMaskBatchedElementParameters;
	FBatchedElements* BrushMaskBatchedElements = nullptr;
	FHitProxyId BrushMaskHitProxyId;
	FTextureRenderTargetResource* BrushMaskRenderTargetResource = nullptr;

	if (bEnableSeamPainting)
	{
		BrushMaskRenderTargetResource = TextureData->BrushMaskRenderTargetTexture->GameThread_GetRenderTargetResource();
		check(BrushMaskRenderTargetResource != nullptr);

		// Create a canvas for the brush mask rendertarget and clear it to black.
		BrushMaskCanvas = TSharedPtr<FCanvas>(new FCanvas(BrushMaskRenderTargetResource, nullptr, FGameTime(), FeatureLevel));
		BrushMaskCanvas->Clear(FLinearColor::Black);

		// Parameters for the mask
		MeshPaintMaskBatchedElementParameters = TRefCountPtr< FMeshPaintBatchedElementParameters >(new FMeshPaintBatchedElementParameters());
		{
			MeshPaintMaskBatchedElementParameters->ShaderParams.PaintBrushTexture = TextureData->PaintBrushRenderTargetTexture;
			if (LastParams)
			{
				MeshPaintMaskBatchedElementParameters->ShaderParams.PaintBrushDirectionVector = InParams.BrushPosition2D - LastParams->BrushPosition2D;
				MeshPaintMaskBatchedElementParameters->ShaderParams.bRotateBrushTowardsDirection = TextureProperties->bRotateBrushTowardsDirection;
			}
			else
			{
				MeshPaintMaskBatchedElementParameters->ShaderParams.PaintBrushDirectionVector = FVector2f(0.0f, 0.0f);
				MeshPaintMaskBatchedElementParameters->ShaderParams.bRotateBrushTowardsDirection = false;
			}
			MeshPaintMaskBatchedElementParameters->ShaderParams.PaintBrushRotationOffset = TextureProperties->PaintBrushRotationOffset;
			MeshPaintMaskBatchedElementParameters->ShaderParams.bUseFillBucket = InParams.bUseFillBucket;
			MeshPaintMaskBatchedElementParameters->ShaderParams.CloneTexture = TextureData->PaintRenderTargetTexture;
			MeshPaintMaskBatchedElementParameters->ShaderParams.WorldToBrushMatrix = WorldToBrushMatrix;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushRadius = InParams.InnerBrushRadius + InParams.BrushRadialFalloffRange;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushRadialFalloffRange = InParams.BrushRadialFalloffRange;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushDepth = InParams.InnerBrushDepth + InParams.BrushDepthFalloffRange;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushDepthFalloffRange = InParams.BrushDepthFalloffRange;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushStrength = InParams.BrushStrength;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BrushColor = InParams.BrushColor;
			MeshPaintMaskBatchedElementParameters->ShaderParams.RedChannelFlag = InParams.bWriteRed;
			MeshPaintMaskBatchedElementParameters->ShaderParams.GreenChannelFlag = InParams.bWriteGreen;
			MeshPaintMaskBatchedElementParameters->ShaderParams.BlueChannelFlag = InParams.bWriteBlue;
			MeshPaintMaskBatchedElementParameters->ShaderParams.AlphaChannelFlag = InParams.bWriteAlpha;
			MeshPaintMaskBatchedElementParameters->ShaderParams.GenerateMaskFlag = true;
		}

		BrushMaskBatchedElements = BrushMaskCanvas->GetBatchedElements(FCanvas::ET_Triangle, MeshPaintMaskBatchedElementParameters, nullptr, SE_BLEND_Opaque);
		BrushMaskBatchedElements->AddReserveVertices(InInfluencedTriangles.Num() * 3);
		BrushMaskBatchedElements->AddReserveTriangles(InInfluencedTriangles.Num(), nullptr, SE_BLEND_Opaque);

		BrushMaskHitProxyId = BrushMaskCanvas->GetHitProxyId();
	}

	// Process the influenced triangles - storing off a large list is much slower than processing in a single loop
	for (int32 CurIndex = 0; CurIndex < InInfluencedTriangles.Num(); ++CurIndex)
	{
		FTexturePaintTriangleInfo& CurTriangle = InInfluencedTriangles[CurIndex];

		FVector2D UVMin(99999.9f, 99999.9f);
		FVector2D UVMax(-99999.9f, -99999.9f);

		// Transform the triangle and update the UV bounds
		for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
		{
			// Update bounds
			float U = CurTriangle.TriUVs[TriVertexNum].X;
			float V = CurTriangle.TriUVs[TriVertexNum].Y;

			if (U < UVMin.X)
			{
				UVMin.X = U;
			}
			if (U > UVMax.X)
			{
				UVMax.X = U;
			}
			if (V < UVMin.Y)
			{
				UVMin.Y = V;
			}
			if (V > UVMax.Y)
			{
				UVMax.Y = V;
			}
		}

		// If the triangle lies entirely outside of the 0.0-1.0 range, we'll transpose it back
		FVector2D UVOffset(0.0f, 0.0f);
		if (UVMax.X > 1.0f)
		{
			UVOffset.X = -FMath::FloorToFloat(UVMin.X);
		}
		else if (UVMin.X < 0.0f)
		{
			UVOffset.X = 1.0f + FMath::FloorToFloat(-UVMax.X);
		}

		if (UVMax.Y > 1.0f)
		{
			UVOffset.Y = -FMath::FloorToFloat(UVMin.Y);
		}
		else if (UVMin.Y < 0.0f)
		{
			UVOffset.Y = 1.0f + FMath::FloorToFloat(-UVMax.Y);
		}

		// Note that we "wrap" the texture coordinates here to handle the case where the user
		// is painting on a tiling texture, or with the UVs out of bounds.  Ideally all of the
		// UVs would be in the 0.0 - 1.0 range but sometimes content isn't setup that way.
		// @todo MeshPaint: Handle triangles that cross the 0.0-1.0 UV boundary?
		for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
		{
			CurTriangle.TriUVs[TriVertexNum].X += UVOffset.X;
			CurTriangle.TriUVs[TriVertexNum].Y += UVOffset.Y;

			// @todo: Need any half-texel offset adjustments here? Some info about offsets and MSAA here: http://drilian.com/2008/11/25/understanding-half-pixel-and-half-texel-offsets/
			// @todo: MeshPaint: Screen-space texture coords: http://diaryofagraphicsprogrammer.blogspot.com/2008/09/calculating-screen-space-texture.html
			CurTriangle.TrianglePoints[TriVertexNum].X = CurTriangle.TriUVs[TriVertexNum].X * TextureData->PaintRenderTargetTexture->GetSurfaceWidth();
			CurTriangle.TrianglePoints[TriVertexNum].Y = CurTriangle.TriUVs[TriVertexNum].Y * TextureData->PaintRenderTargetTexture->GetSurfaceHeight();
		}

		// Vertex positions
		FVector4 Vert0(CurTriangle.TrianglePoints[0].X, CurTriangle.TrianglePoints[0].Y, 0, 1);
		FVector4 Vert1(CurTriangle.TrianglePoints[1].X, CurTriangle.TrianglePoints[1].Y, 0, 1);
		FVector4 Vert2(CurTriangle.TrianglePoints[2].X, CurTriangle.TrianglePoints[2].Y, 0, 1);

		// Vertex color
		FLinearColor Col0(CurTriangle.TriVertices[0].X, CurTriangle.TriVertices[0].Y, CurTriangle.TriVertices[0].Z);
		FLinearColor Col1(CurTriangle.TriVertices[1].X, CurTriangle.TriVertices[1].Y, CurTriangle.TriVertices[1].Z);
		FLinearColor Col2(CurTriangle.TriVertices[2].X, CurTriangle.TriVertices[2].Y, CurTriangle.TriVertices[2].Z);

		// Brush Paint triangle
		{
			int32 V0 = BrushPaintBatchedElements->AddVertex(Vert0, CurTriangle.TriUVs[0], Col0, BrushPaintHitProxyId);
			int32 V1 = BrushPaintBatchedElements->AddVertex(Vert1, CurTriangle.TriUVs[1], Col1, BrushPaintHitProxyId);
			int32 V2 = BrushPaintBatchedElements->AddVertex(Vert2, CurTriangle.TriUVs[2], Col2, BrushPaintHitProxyId);

			BrushPaintBatchedElements->AddTriangle(V0, V1, V2, MeshPaintBatchedElementParameters, SE_BLEND_Opaque);
		}

		// Brush Mask triangle
		if (bEnableSeamPainting)
		{
			int32 V0 = BrushMaskBatchedElements->AddVertex(Vert0, CurTriangle.TriUVs[0], Col0, BrushMaskHitProxyId);
			int32 V1 = BrushMaskBatchedElements->AddVertex(Vert1, CurTriangle.TriUVs[1], Col1, BrushMaskHitProxyId);
			int32 V2 = BrushMaskBatchedElements->AddVertex(Vert2, CurTriangle.TriUVs[2], Col2, BrushMaskHitProxyId);

			BrushMaskBatchedElements->AddTriangle(V0, V1, V2, MeshPaintMaskBatchedElementParameters, SE_BLEND_Opaque);
		}
	}

	// Tell the rendering thread to draw any remaining batched elements
	{
		BrushPaintCanvas.Flush_GameThread(true);

		TextureData->bIsPaintingTexture2DModified = true;
		TextureData->PaintedComponents.AddUnique(MeshComponent);
	}

	ENQUEUE_RENDER_COMMAND(UpdateMeshPaintRTCommand1)(
		[BrushRenderTargetResource](FRHICommandListImmediate& RHICmdList)
	{
		TransitionAndCopyTexture(RHICmdList, BrushRenderTargetResource->GetRenderTargetTexture(), BrushRenderTargetResource->TextureRHI, {});
	});

	if (bEnableSeamPainting)
	{
		BrushMaskCanvas->Flush_GameThread(true);

		ENQUEUE_RENDER_COMMAND(UpdateMeshPaintRTCommand2)(
			[BrushMaskRenderTargetResource](FRHICommandListImmediate& RHICmdList)
		{
			TransitionAndCopyTexture(RHICmdList, BrushMaskRenderTargetResource->GetRenderTargetTexture(), BrushMaskRenderTargetResource->TextureRHI, {});
		});
	}

	if (!bEnableSeamPainting)
	{
		// Seam painting is not enabled so we just copy our delta paint info to the paint target.
		UTexturePaintToolset::CopyTextureToRenderTargetTexture(TextureData->BrushRenderTargetTexture, TextureData->PaintRenderTargetTexture, FeatureLevel);
	}
	else
	{
		// Constants used for generating quads across entire paint rendertarget
		const float MinU = 0.0f;
		const float MinV = 0.0f;
		const float MaxU = 1.0f;
		const float MaxV = 1.0f;
		const float MinX = 0.0f;
		const float MinY = 0.0f;
		const float MaxX = TextureData->PaintRenderTargetTexture->GetSurfaceWidth();
		const float MaxY = TextureData->PaintRenderTargetTexture->GetSurfaceHeight();

		if (TextureData->SeamMaskComponent != MeshComponent)
		{
			// Generate the texture seam mask.  This is a slow operation when the object has many triangles so we try to only do it once when painting is started.
			// @todo MeshPaint: We only store one seam mask, so when we are painting to a texture asset with multi-select we will end up re-rendering the mask each time the brush crosses component boundaries.
			// Better could be to store a mask per component instead, but still lazily generate them on demand.
			UTexturePaintToolset::GenerateSeamMask(MeshComponent, UVChannel, TextureData->SeamMaskRenderTargetTexture, TextureData->PaintingTexture2D, TextureData->PaintRenderTargetTexture);
			TextureData->SeamMaskComponent = MeshComponent;
		}

		FTextureRenderTargetResource* RenderTargetResource = TextureData->PaintRenderTargetTexture->GameThread_GetRenderTargetResource();
		check(RenderTargetResource != nullptr);
		// Dilate the paint stroke into the texture seams.
		{
			// Create a canvas for the render target.
			FCanvas Canvas3(RenderTargetResource, nullptr, FGameTime(), FeatureLevel);

			TRefCountPtr< FMeshPaintDilateBatchedElementParameters > MeshPaintDilateBatchedElementParameters(new FMeshPaintDilateBatchedElementParameters());
			{
				MeshPaintDilateBatchedElementParameters->ShaderParams.Texture0 = TextureData->BrushRenderTargetTexture;
				MeshPaintDilateBatchedElementParameters->ShaderParams.Texture1 = TextureData->SeamMaskRenderTargetTexture;
				MeshPaintDilateBatchedElementParameters->ShaderParams.Texture2 = TextureData->BrushMaskRenderTargetTexture;
				MeshPaintDilateBatchedElementParameters->ShaderParams.WidthPixelOffset = (float)(1.0f / TextureData->PaintRenderTargetTexture->GetSurfaceWidth());
				MeshPaintDilateBatchedElementParameters->ShaderParams.HeightPixelOffset = (float)(1.0f / TextureData->PaintRenderTargetTexture->GetSurfaceHeight());
			}

			// Draw a quad to copy the texture over to the render target
			TArray< FCanvasUVTri >	TriangleList;
			FCanvasUVTri SingleTri;
			SingleTri.V0_Pos = FVector2D(MinX, MinY);
			SingleTri.V0_UV = FVector2D(MinU, MinV);
			SingleTri.V0_Color = FLinearColor::White;

			SingleTri.V1_Pos = FVector2D(MaxX, MinY);
			SingleTri.V1_UV = FVector2D(MaxU, MinV);
			SingleTri.V1_Color = FLinearColor::White;

			SingleTri.V2_Pos = FVector2D(MaxX, MaxY);
			SingleTri.V2_UV = FVector2D(MaxU, MaxV);
			SingleTri.V2_Color = FLinearColor::White;
			TriangleList.Add(SingleTri);

			SingleTri.V0_Pos = FVector2D(MaxX, MaxY);
			SingleTri.V0_UV = FVector2D(MaxU, MaxV);
			SingleTri.V0_Color = FLinearColor::White;

			SingleTri.V1_Pos = FVector2D(MinX, MaxY);
			SingleTri.V1_UV = FVector2D(MinU, MaxV);
			SingleTri.V1_Color = FLinearColor::White;

			SingleTri.V2_Pos = FVector2D(MinX, MinY);
			SingleTri.V2_UV = FVector2D(MinU, MinV);
			SingleTri.V2_Color = FLinearColor::White;
			TriangleList.Add(SingleTri);

			FCanvasTriangleItem TriItemList(TriangleList, nullptr);
			TriItemList.BatchedElementParameters = MeshPaintDilateBatchedElementParameters;
			TriItemList.BlendMode = SE_BLEND_Opaque;
			Canvas3.DrawItem(TriItemList);

			// Tell the rendering thread to draw any remaining batched elements
			Canvas3.Flush_GameThread(true);
		}

		ENQUEUE_RENDER_COMMAND(UpdateMeshPaintRTCommand3)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
		{
			TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
		});
	}

	// Need to flush the virtual texture adapter since we just updated the painting render target.
	if (TextureData->PaintRenderTargetTextureAdapter)
	{
		TextureData->PaintRenderTargetTextureAdapter->Flush(FBox2f(FVector2f(0, 0), FVector2f(1, 1)));
	}

	// Need to invalidate runtime virtual textures if paint texture is used in a material that writes to one.
	if (bRequiresRuntimeVirtualTextureUpdates)
	{
		FBoxSphereBounds BrushBounds(FSphere(InParams.BrushPosition, InParams.InnerBrushRadius + InParams.BrushRadialFalloffRange));
		for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			It->Invalidate(BrushBounds, EVTInvalidatePriority::High);
		}
	}
}


void UMeshTexturePaintingTool::FinishPaintingTexture()
{
	if (FPaintTexture2DData* TextureData = GetPaintTargetData(PaintingTexture2D))
	{
		// Apply the texture
		if (TextureData->bIsPaintingTexture2DModified == true)
		{
			const int32 TexWidth = TextureData->PaintRenderTargetTexture->SizeX;
			const int32 TexHeight = TextureData->PaintRenderTargetTexture->SizeY;
			TArray< FColor > TexturePixels;
			TexturePixels.AddUninitialized(TexWidth * TexHeight);

			// Copy the contents of the remote texture to system memory

			FlushRenderingCommands();
			// NOTE: You are normally not allowed to dereference this pointer on the game thread! Normally you can only pass the pointer around and
			//  check for NULLness.  We do it in this context, however, and it is only ok because this does not happen every frame and we make sure to flush the
			//  rendering thread.
			FTextureRenderTargetResource* RenderTargetResource = TextureData->PaintRenderTargetTexture->GameThread_GetRenderTargetResource();
			check(RenderTargetResource != nullptr);

			FReadSurfaceDataFlags Flags;
			Flags.SetLinearToGamma(PaintingTexture2D->SRGB);
			RenderTargetResource->ReadPixels(TexturePixels, Flags);

			// For undo
			TextureData->PaintingTexture2D->SetFlags(RF_Transactional);
			TextureData->PaintingTexture2D->PreEditChange(nullptr);

			// Store source art
			FImageView ImageView(TexturePixels.GetData(), TexWidth, TexHeight, EGammaSpace::sRGB);
			TextureData->PaintingTexture2D->Source.Init(ImageView);

			TextureData->PaintingTexture2D->bHasBeenPaintedInEditor = true;

			// Update the texture (generate mips, compress if needed)
			TextureData->PaintingTexture2D->PostEditChange();

			TextureData->bIsPaintingTexture2DModified = false;

			for (UMeshComponent* PaintedComponent : TextureData->PaintedComponents)
			{
				OnPaintingFinishedDelegate.ExecuteIfBound(PaintedComponent);
			}
			TextureData->PaintedComponents.Reset();
		}
	}

	PaintingTexture2D = nullptr;
}

void UMeshTexturePaintingTool::ClearAllTextureOverrides(FMaterialUpdateContext& MaterialUpdateContext)
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		/** Remove all texture overrides which are currently stored and active */
		for (decltype(PaintTargetData)::TIterator It(PaintTargetData); It; ++It)
		{
			FPaintTexture2DData* TextureData = &It.Value();

			for (UMeshComponent* MeshComponent : TextureData->TextureOverrideComponents)
			{
				if (TSharedPtr<IMeshPaintComponentAdapter> PaintAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent))
				{
					PaintAdapter->ApplyOrRemoveTextureOverride(TextureData->PaintingTexture2D, nullptr, MaterialUpdateContext);
				}
			}
	
			TextureData->TextureOverrideComponents.Empty();
		}
	}
}

void UMeshTexturePaintingTool::SetAllTextureOverrides(FMaterialUpdateContext& MaterialUpdateContext)
{
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	TArray<UMeshComponent*> SelectedMeshComponents = MeshPaintingSubsystem->GetSelectedMeshComponents();

	for (FPaintableTexture const& PaintableTexture : PaintableTextures)
	{
		// Apply the overrides only to the components that we are painting with this texture.
		TArray<UMeshComponent*, TInlineAllocator<8>> PaintableMeshComponents;
		for (UMeshComponent* MeshComponent : SelectedMeshComponents)
		{
			if (CanPaintTextureToComponent(PaintableTexture.Texture, MeshComponent))
			{
				TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);
				if (IsMeshAdapterSupported(MeshAdapter))
				{
					PaintableMeshComponents.Add(MeshComponent);
				}
			}
		}

		if (PaintableMeshComponents.Num() == 0)
		{
			continue;
		}

		UTexture2D* Texture2D = Cast<UTexture2D>(PaintableTexture.Texture);
		if (Texture2D == nullptr)
		{
			continue;
		}
	
		Texture2D->BlockOnAnyAsyncBuild();

		// Create the render target to paint on.
		FPaintTexture2DData* TextureData = AddPaintTargetData(Texture2D);

		const int32 TextureWidth = Texture2D->Source.GetSizeX();
		const int32 TextureHeight = Texture2D->Source.GetSizeY();

		if (TextureData->PaintRenderTargetTexture == nullptr ||
			TextureData->PaintRenderTargetTexture->GetSurfaceWidth() != TextureWidth ||
			TextureData->PaintRenderTargetTexture->GetSurfaceHeight() != TextureHeight)
		{
			TextureData->PaintRenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);
			TextureData->PaintRenderTargetTexture->bNeedsTwoCopies = true;
			const bool bForceLinearGamma = true;
			TextureData->PaintRenderTargetTexture->InitCustomFormat(TextureWidth, TextureHeight, PF_A16B16G16R16, bForceLinearGamma);
			TextureData->PaintRenderTargetTexture->UpdateResourceImmediate();

			TextureData->PaintRenderTargetTextureAdapter = nullptr; 
			if (TextureData->PaintingTexture2D->IsCurrentlyVirtualTextured())
			{
				// Virtual textures can't just swap in a render target in their material, so we use a virtual texture adapter.
				FVirtualTextureBuildSettings VirtualTextureBuildSettings;
				TextureData->PaintingTexture2D->GetVirtualTextureBuildSettings(VirtualTextureBuildSettings);

				TextureData->PaintRenderTargetTextureAdapter = NewObject<UVirtualTextureAdapter>(GetTransientPackage(), NAME_None, RF_Transient);
				TextureData->PaintRenderTargetTextureAdapter->Texture = TextureData->PaintRenderTargetTexture;
				TextureData->PaintRenderTargetTextureAdapter->OverrideWithTextureFormat = Texture2D;
				TextureData->PaintRenderTargetTextureAdapter->bUseDefaultTileSizes = false;
				TextureData->PaintRenderTargetTextureAdapter->TileSize = VirtualTextureBuildSettings.TileSize;
				TextureData->PaintRenderTargetTextureAdapter->TileBorderSize = VirtualTextureBuildSettings.TileBorderSize;
				TextureData->PaintRenderTargetTextureAdapter->UpdateResource();
			}
		}

		TextureData->PaintRenderTargetTexture->AddressX = Texture2D->AddressX;
		TextureData->PaintRenderTargetTexture->AddressY = Texture2D->AddressY;

		// Initialize the render target with the texture contents.
		UTexturePaintToolset::SetupInitialRenderTargetData(TextureData->PaintingTexture2D, TextureData->PaintRenderTargetTexture);

		// Need to flush the virtual texture adapter since we just updated the painting render target.
		if (TextureData->PaintRenderTargetTextureAdapter)
		{
			TextureData->PaintRenderTargetTextureAdapter->Flush(FBox2f(FVector2f(0, 0), FVector2f(1, 1)));
		}

		// Apply the overrides.
		for (UMeshComponent* MeshComponent : PaintableMeshComponents)
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);
			AddTextureOverrideToComponent(*TextureData, MeshComponent, MeshAdapter.Get(), MaterialUpdateContext);
		}
	}
}

void UMeshTexturePaintingTool::FloodCurrentPaintTexture()
{
	bRequestPaintBucketFill = true;
}


UMeshTextureColorPaintingTool::UMeshTextureColorPaintingTool()
{
	PropertyClass = UMeshTextureColorPaintingToolProperties::StaticClass();
}

void UMeshTextureColorPaintingTool::Setup()
{
	Super::Setup();
	ColorProperties = Cast<UMeshTextureColorPaintingToolProperties>(BrushProperties);

	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		// Create a dummy mesh paint virtual texture for the lifetime of the paint tool.
		// This keeps at least one virtual texture alive during painting.
		// Otherwise, if there is only one "real" virtual texture in the scene and we paint on it, 
		// it will be deallocted for one or two frames during texture compilation after each paint stroke.
		// For those frames there would be _no_ remainging allocated VTs to use for the FMeshPaintVirtualTextureSceneExtension
		// and which would leave no page table bound for sampling the virtual texture adaptor that wraps the 
		// painting render target. That would result in a flicker where the lack of page table means the mesh paint
		// virtual texture gets its fallback color when sampling.
		// Holding this dummy texture prevents that from happening.
		MeshPaintDummyTexture = MeshPaintingSubsystem->CreateMeshPaintTexture(this, 1);
	}

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTextureColorPaintTool", "Paint colors to the Mesh Paint Texture object stored on mesh components."),
		EToolMessageLevel::UserNotification);
}

bool UMeshTextureColorPaintingTool::IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const
{
	return MeshAdapter.IsValid() ? MeshAdapter->SupportsTextureColorPaint() : false;
}

UTexture2D* UMeshTextureColorPaintingTool::GetSelectedPaintTexture(UMeshComponent const* InMeshComponent) const
{
	return Cast<UTexture2D>(InMeshComponent->GetMeshPaintTexture());
}

int32 UMeshTextureColorPaintingTool::GetSelectedUVChannel(UMeshComponent const* InMeshComponent) const
{
	return InMeshComponent != nullptr ? InMeshComponent->GetMeshPaintTextureCoordinateIndex() : 0;
}

void UMeshTextureColorPaintingTool::GetModifiedTexturesToSave(TArray<UObject*>& OutTexturesToSave) const
{
	for (FPaintableTexture const& PaintableTexture : PaintableTextures)
	{
		if (PaintableTexture.Texture->GetOutermost()->IsDirty())
		{
			OutTexturesToSave.Add(PaintableTexture.Texture);
		}
	}
}

void UMeshTextureColorPaintingTool::CacheTexturePaintData()
{
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		PaintableTextures.Empty();

		TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();
		for (UMeshComponent const* Component : PaintableComponents)
		{
			int32 DummyDefaultIndex;
			TSharedPtr<IMeshPaintComponentAdapter> Adapter = MeshPaintingSubsystem->GetAdapterForComponent(Component);
			UTexturePaintToolset::RetrieveTexturesForComponent(Component, Adapter.Get(), DummyDefaultIndex, PaintableTextures);
		}

		PaintableTextures.RemoveAll([](FPaintableTexture const& PaintableTexture) { return !PaintableTexture.bIsMeshTexture; });
	}
}

bool UMeshTextureColorPaintingTool::CanPaintTextureToComponent(UTexture* InTexture, UMeshComponent const* InMeshComponent) const
{
	return InMeshComponent->GetMeshPaintTexture() == InTexture;
}


UMeshTextureAssetPaintingTool::UMeshTextureAssetPaintingTool()
{
	PropertyClass = UMeshTextureAssetPaintingToolProperties::StaticClass();
}

void UMeshTextureAssetPaintingTool::Setup()
{
	Super::Setup();
	AssetProperties = Cast<UMeshTextureAssetPaintingToolProperties>(BrushProperties);

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTexturePaintTool", "The Texture Weight Painting mode enables you to paint on textures and access available properties while doing so ."),
		EToolMessageLevel::UserNotification);
}

bool UMeshTextureAssetPaintingTool::IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const
{
	return MeshAdapter.IsValid() ? MeshAdapter->SupportsTexturePaint() : false;
}

UTexture2D* UMeshTextureAssetPaintingTool::GetSelectedPaintTexture(UMeshComponent const* InMeshComponent) const
{
	return AssetProperties->PaintTexture;
}

int32 UMeshTextureAssetPaintingTool::GetSelectedUVChannel(UMeshComponent const* InMeshComponent) const
{
	return AssetProperties->UVChannel;
}

void UMeshTextureAssetPaintingTool::GetModifiedTexturesToSave(TArray<UObject*>& OutTexturesToSave) const
{
	if (AssetProperties->PaintTexture != nullptr && AssetProperties->PaintTexture->GetOutermost()->IsDirty())
	{
		OutTexturesToSave.Add(AssetProperties->PaintTexture);
	}
}

bool UMeshTextureAssetPaintingTool::ShouldFilterTextureAsset(const FAssetData& AssetData) const
{
	FSoftObjectPath Path = AssetData.GetSoftObjectPath();
	return !(PaintableTextures.ContainsByPredicate([&Path](const FPaintableTexture& Texture) { return FSoftObjectPath(Texture.Texture) == Path; }));
}

void UMeshTextureAssetPaintingTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);

	// Need to update the currently painted texture.
	bool bUpdateTextureOverrides = false;
	// Need to recache PaintableTextures.
	bool bUpdateCachedTexturePaintData = false;
	
	if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTextureAssetPaintingToolProperties, ChannelSet))
	{
		// Changing channel set changes the PaintableTextures.
		bUpdateCachedTexturePaintData = true;
	}

	if (Property &&	Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTextureAssetPaintingToolProperties, ChannelName))
	{
		UTexture2D* ChannelPaintTexture = nullptr;
		if (AssetProperties->ChannelSet)
		{
			if (FMeshPaintChannelDesc const* Channel = AssetProperties->ChannelSet->GetChannelDesc(AssetProperties->ChannelName))
			{
				ChannelPaintTexture = Channel->PaintTexture;
			}
		}
		
		bUpdateTextureOverrides |= AssetProperties->PaintTexture != ChannelPaintTexture || AssetProperties->UVChannel != 0;
		bUpdateCachedTexturePaintData |= PaintableTextures.Num() != 1 || PaintableTextures[0].Texture != ChannelPaintTexture;

		AssetProperties->PaintTexture = ChannelPaintTexture;
		AssetProperties->UVChannel = 0;
	}

	if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTextureAssetPaintingToolProperties, PaintTexture))
	{
		// Find the selected texture and apply it's UV channel.
		for (FPaintableTexture& PaintableTexture : PaintableTextures)
		{
			if (PaintableTexture.Texture == AssetProperties->PaintTexture)
			{
				AssetProperties->UVChannel = PaintableTexture.UVChannelIndex;
				break;
			}
		}
		
		// Need to recreate the render target overrides with the newly selected texture.
		bUpdateTextureOverrides = true;
	}
	
	if (bUpdateTextureOverrides || bUpdateCachedTexturePaintData)
	{
		FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default|FMaterialUpdateContext::EOptions::SkipRecacheStaticPermutations);

		ClearAllTextureOverrides(MaterialUpdateContext);
		if (bUpdateCachedTexturePaintData)
		{
			CacheTexturePaintData();
		}
		SetAllTextureOverrides(MaterialUpdateContext);
	}
}

void UMeshTextureAssetPaintingTool::CacheTexturePaintData()
{
	if (AssetProperties->ChannelSet != nullptr)
	{
		PaintableTextures.Reset();
		AssetProperties->PaintTexture = nullptr;
		AssetProperties->UVChannel = INDEX_NONE;

		if (FMeshPaintChannelDesc const* Channel = AssetProperties->ChannelSet->GetChannelDesc(AssetProperties->ChannelName))
		{
			if (Channel->PaintTexture)
			{
				PaintableTextures.Add(Channel->PaintTexture);
				AssetProperties->PaintTexture = Channel->PaintTexture;
				AssetProperties->UVChannel = 0;
			}
		}

		return;
	}

	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		PaintableTextures.Reset();

		UTexture* DefaultTexture = nullptr;
		int32 DefaultUVChannelIndex = INDEX_NONE;

		TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();

		// Gather textures on first component.
		if (PaintableComponents.Num() > 0)
		{
			TSharedPtr<IMeshPaintComponentAdapter> Adapter = MeshPaintingSubsystem->GetAdapterForComponent(PaintableComponents[0]);
			int32 DefaultTextureIndex = INDEX_NONE;
			UTexturePaintToolset::RetrieveTexturesForComponent(PaintableComponents[0], Adapter.Get(), DefaultTextureIndex, PaintableTextures);
			if (DefaultTexture == nullptr && PaintableTextures.IsValidIndex(DefaultTextureIndex))
			{
				DefaultTexture = PaintableTextures[DefaultTextureIndex].Texture;
				DefaultUVChannelIndex = PaintableTextures[DefaultTextureIndex].UVChannelIndex;
			}
		}

		// If there is more than one component we only want textures that are referenced by ALL selected components.
		for (int32 Index = 1; Index < PaintableComponents.Num(); ++Index)
		{
			UMeshComponent* PaintableComponent = PaintableComponents[Index];
			TSharedPtr<IMeshPaintComponentAdapter> Adapter = MeshPaintingSubsystem->GetAdapterForComponent(PaintableComponent);
			TArray<FPaintableTexture> PerComponentPaintableTextures;
			int32 DefaultTextureIndex = INDEX_NONE;
			UTexturePaintToolset::RetrieveTexturesForComponent(PaintableComponent, Adapter.Get(), DefaultTextureIndex, PerComponentPaintableTextures);
			if (DefaultTexture == nullptr && PerComponentPaintableTextures.IsValidIndex(DefaultTextureIndex))
			{
				DefaultTexture = PerComponentPaintableTextures[DefaultTextureIndex].Texture;
				DefaultUVChannelIndex = PerComponentPaintableTextures[DefaultTextureIndex].UVChannelIndex;
			}

			TArray<FPaintableTexture> CommonPaintableTextures;
			for (FPaintableTexture const& PaintableTexture : PerComponentPaintableTextures)
			{
				if (PaintableTextures.Contains(PaintableTexture))
				{
					CommonPaintableTextures.Add(PaintableTexture);
				}
			}
			PaintableTextures = MoveTemp(CommonPaintableTextures);
		}

		PaintableTextures.RemoveAll([](FPaintableTexture const& PaintableTexture) { return PaintableTexture.bIsMeshTexture; });

		// Ensure that the selection remains valid or is invalidated
		int32 SelectedIndex = INDEX_NONE;

		if (AssetProperties->PaintTexture != nullptr)
		{
			// First try to find fully matching entry, then by texture only (a texture may appear with multiple UV channels).
			SelectedIndex = PaintableTextures.Find(FPaintableTexture(AssetProperties->PaintTexture, AssetProperties->UVChannel));
			if (SelectedIndex == INDEX_NONE)
			{
				SelectedIndex = PaintableTextures.IndexOfByPredicate([&](const FPaintableTexture& Texture) { return Texture.Texture == AssetProperties->PaintTexture; });
			}
		}
		if (SelectedIndex == INDEX_NONE && DefaultTexture != nullptr)
		{
			SelectedIndex = PaintableTextures.Find(FPaintableTexture(DefaultTexture, DefaultUVChannelIndex));
			if (SelectedIndex == INDEX_NONE)
			{
				SelectedIndex = PaintableTextures.IndexOfByPredicate([&](const FPaintableTexture& Texture) { return Texture.Texture == DefaultTexture; });
			}
		}
		if (SelectedIndex == INDEX_NONE && PaintableTextures.Num() > 0)
		{
			SelectedIndex = 0;
		}

		AssetProperties->PaintTexture = SelectedIndex == INDEX_NONE ? nullptr : Cast<UTexture2D>(PaintableTextures[SelectedIndex].Texture);
		AssetProperties->UVChannel = SelectedIndex == INDEX_NONE ? INDEX_NONE : PaintableTextures[SelectedIndex].UVChannelIndex;
	}
}

bool UMeshTextureAssetPaintingTool::CanPaintTextureToComponent(UTexture* InTexture, UMeshComponent const* InMeshComponent) const
{
	return InTexture == AssetProperties->PaintTexture;
}

UTexture* UMeshTextureAssetPaintingTool::GetSelectedPaintTextureWithOverride() const
{
	UTexture* SelectedTexture = AssetProperties->PaintTexture;
	if (AssetProperties->PaintTexture != nullptr)
	{
		FPaintTexture2DData const* TextureData = PaintTargetData.Find(AssetProperties->PaintTexture);
		if (TextureData != nullptr && TextureData->PaintRenderTargetTexture)
		{
			SelectedTexture = TextureData->PaintRenderTargetTexture;
		}
	}
	return SelectedTexture;
}

void UMeshTextureAssetPaintingTool::CycleTextures(int32 Direction)
{
	if (AssetProperties->ChannelSet != nullptr)
	{
		return;
	}
	if (!PaintableTextures.Num())
	{
		return;
	}
	TObjectPtr<UTexture2D>& SelectedTexture = AssetProperties->PaintTexture;
	const int32 TextureIndex = (SelectedTexture != nullptr) ? PaintableTextures.IndexOfByKey(SelectedTexture) : 0;
	if (TextureIndex != INDEX_NONE)
	{
		int32 NewTextureIndex = TextureIndex + Direction;
		if (NewTextureIndex < 0)
		{
			NewTextureIndex += PaintableTextures.Num();
		}
		NewTextureIndex %= PaintableTextures.Num();

		if (PaintableTextures.IsValidIndex(NewTextureIndex))
		{
			SelectedTexture = (UTexture2D*)PaintableTextures[NewTextureIndex].Texture;
		}
	}
}

void UMeshTextureAssetPaintingTool::ApplyWorldChannelUVs(TArrayView<FTexturePaintTriangleInfo> TriangleInfo) const
{
	if (AssetProperties->ChannelSet)
	{
		if (FMeshPaintChannelDesc const* Channel = AssetProperties->ChannelSet->GetChannelDesc(AssetProperties->ChannelName))
		{
			// Scale is stored as UV->World to make UI easier to use (it's difficult to enter very small numbers with precision).
			const float WorldToUVScaleX = FMath::IsNearlyZero(Channel->UVToWorldScale.X) ? 1.f : 1.f / Channel->UVToWorldScale.X;
			const float WorldToUVScaleY = FMath::IsNearlyZero(Channel->UVToWorldScale.Y) ? 1.f : 1.f / Channel->UVToWorldScale.Y;

			for (FTexturePaintTriangleInfo& Info : TriangleInfo)
			{
				Info.TriUVs[0].X = FMath::Clamp(Info.TriVertices[0].X * WorldToUVScaleX + Channel->WorldToUVBias.X, Channel->ClampUVMin.X, Channel->ClampUVMax.X);
				Info.TriUVs[0].Y = FMath::Clamp(Info.TriVertices[0].Y * WorldToUVScaleY + Channel->WorldToUVBias.Y, Channel->ClampUVMin.Y, Channel->ClampUVMax.Y);
				Info.TriUVs[1].X = FMath::Clamp(Info.TriVertices[1].X * WorldToUVScaleX + Channel->WorldToUVBias.X, Channel->ClampUVMin.X, Channel->ClampUVMax.X);
				Info.TriUVs[1].Y = FMath::Clamp(Info.TriVertices[1].Y * WorldToUVScaleY + Channel->WorldToUVBias.Y, Channel->ClampUVMin.Y, Channel->ClampUVMax.Y);
				Info.TriUVs[2].X = FMath::Clamp(Info.TriVertices[2].X * WorldToUVScaleX + Channel->WorldToUVBias.X, Channel->ClampUVMin.X, Channel->ClampUVMax.X);
				Info.TriUVs[2].Y = FMath::Clamp(Info.TriVertices[2].Y * WorldToUVScaleY + Channel->WorldToUVBias.Y, Channel->ClampUVMin.Y, Channel->ClampUVMax.Y);
			}
		}
	}
}

TArray<FName> UMeshTextureAssetPaintingToolProperties::GetChannelNames() const
{
	TArray<FName> ChannelNames;
	if (ChannelSet != nullptr)
	{
		for (FMeshPaintChannelDesc const& Channel : ChannelSet->Channels)
		{
			ChannelNames.AddUnique(Channel.ChannelName);
		}
	}
	return ChannelNames;
}

bool UMeshTextureAssetPaintingToolProperties::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);
	if (InProperty != nullptr && 
		(InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTexturePaintingToolProperties, bWriteRed) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTexturePaintingToolProperties, bWriteGreen) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTexturePaintingToolProperties, bWriteBlue) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTexturePaintingToolProperties, bWriteAlpha) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTexturePaintingToolProperties, bEnableSeamPainting)))
	{
		bCanEdit &= ChannelSet == nullptr;
	}
	return bCanEdit;
}

void UMeshTextureAssetPaintingToolProperties::GetWriteMask(bool& bOutWriteRed, bool& bOutWriteGreen, bool& bOutWriteBlue, bool& bOutWriteAlpha) const
{
	if (ChannelSet == nullptr)
	{
		Super::GetWriteMask(bOutWriteRed, bOutWriteGreen, bOutWriteBlue, bOutWriteAlpha);
		return;
	}

	if (FMeshPaintChannelDesc const* Channel = ChannelSet->GetChannelDesc(ChannelName))
	{
		bOutWriteRed = Channel->bWriteRed;
		bOutWriteGreen = Channel->bWriteGreen;
		bOutWriteBlue = Channel->bWriteBlue;
		bOutWriteAlpha = Channel->bWriteAlpha;
		return;
	}
		
	// Didn't find channel so disable all writes.
	bOutWriteRed = bOutWriteGreen = bOutWriteBlue = bOutWriteAlpha = false;
}

#undef LOCTEXT_NAMESPACE
