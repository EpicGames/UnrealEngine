// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexPaintingTool.h"
#include "InteractiveToolManager.h"
#include "Components/MeshComponent.h"
#include "IMeshPaintComponentAdapter.h"
#include "ComponentReregisterContext.h"
#include "ToolDataVisualizer.h"
#include "MeshPaintHelpers.h"
#include "BaseGizmos/BrushStampIndicator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexPaintingTool)


#define LOCTEXT_NAMESPACE "MeshVertexBrush"

/*
 * ToolBuilder
 */

bool UMeshVertexColorPaintingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetSelectionSupportsVertexPaint();
}

UInteractiveTool* UMeshVertexColorPaintingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMeshVertexColorPaintingTool* NewTool = NewObject<UMeshVertexColorPaintingTool>(SceneState.ToolManager);
	return NewTool;
}

bool UMeshVertexWeightPaintingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->GetSelectionSupportsVertexPaint();
}

UInteractiveTool* UMeshVertexWeightPaintingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMeshVertexWeightPaintingTool* NewTool = NewObject<UMeshVertexWeightPaintingTool>(SceneState.ToolManager);
	return NewTool;
}


/*
 * Tool
 */

UMeshVertexPaintingToolProperties::UMeshVertexPaintingToolProperties()
	:UMeshPaintingToolProperties(),
	VertexPreviewSize(6.0f)
{
}

UMeshVertexPaintingTool::UMeshVertexPaintingTool()
{
	PropertyClass = UMeshVertexPaintingToolProperties::StaticClass();
}

bool UMeshVertexPaintingTool::IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const
{
	return MeshAdapter.IsValid() ? MeshAdapter->SupportsVertexPaint() : false;
}

void UMeshVertexPaintingTool::Setup()
{
	Super::Setup();

	bResultValid = false;
	bStampPending = false;

	FMeshPaintToolSettingHelpers::RestorePropertiesForClassHeirachy(this, BrushProperties);
	VertexProperties = Cast<UMeshVertexPaintingToolProperties>(BrushProperties);

	// Needed after restoring properties because the brush radius may be an output
	// property based on selection, so we shouldn't use the last stored value there.
	// We wouldn't have this problem if we restore properties before getting
	// BrushRelativeSizeRange, but that happens in the Super::Setup() call earlier.
	RecalculateBrushRadius(); 

	BrushStampIndicator->LineColor = FLinearColor::Green;

	SelectionMechanic = NewObject<UMeshPaintSelectionMechanic>(this);
	SelectionMechanic->Setup(this);
}


void UMeshVertexPaintingTool::Shutdown(EToolShutdownType ShutdownType)
{
	//If we're painting vertex colors then propagate the painting done on LOD0 to all lower LODs. 
	//Then stop forcing the LOD level of the mesh to LOD0.
	ApplyForcedLODIndex(-1);

	FinishPainting();
	
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		MeshPaintingSubsystem->Refresh();
	}

	FMeshPaintToolSettingHelpers::SavePropertiesForClassHeirachy(this, BrushProperties);

	Super::Shutdown(ShutdownType);
}

void UMeshVertexPaintingTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	Super::Render(RenderAPI);
	FToolDataVisualizer Draw;
	Draw.BeginFrame(RenderAPI);
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem && LastBestHitResult.Component != nullptr && MeshPaintingSubsystem->GetPaintableMeshComponents().Num() > 0)
	{
		BrushStampIndicator->bDrawIndicatorLines = true;
		static float WidgetLineThickness = 1.0f;
		static FLinearColor VertexPointColor = FLinearColor::White;
		static FLinearColor	HoverVertexPointColor = FLinearColor(0.3f, 1.0f, 0.3f);
		const float NormalLineSize(BrushProperties->BrushRadius * 0.35f);	// Make the normal line length a function of brush size
		static const FLinearColor NormalLineColor(0.3f, 1.0f, 0.3f);
		const FLinearColor BrushCueColor = (bArePainting ? FLinearColor(1.0f, 1.0f, 0.3f) : FLinearColor(0.3f, 1.0f, 0.3f));
 		const FLinearColor InnerBrushCueColor = (bArePainting ? FLinearColor(0.5f, 0.5f, 0.1f) : FLinearColor(0.1f, 0.5f, 0.1f));
		const float PointDrawSize = VertexProperties->VertexPreviewSize;
		// Draw trace surface normal
		const FVector NormalLineEnd(LastBestHitResult.Location + LastBestHitResult.Normal * NormalLineSize);
		Draw.DrawLine(FVector(LastBestHitResult.Location), NormalLineEnd, NormalLineColor, WidgetLineThickness);

		for (UMeshComponent* CurrentComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(Cast<UMeshComponent>(CurrentComponent));

			if (MeshAdapter && MeshAdapter->SupportsVertexPaint())
			{
				const FMatrix ComponentToWorldMatrix = MeshAdapter->GetComponentToWorldMatrix();
				FViewCameraState CameraState;
				GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
				const FVector ComponentSpaceCameraPosition(ComponentToWorldMatrix.InverseTransformPosition(CameraState.Position));
				const FVector ComponentSpaceBrushPosition(ComponentToWorldMatrix.InverseTransformPosition(LastBestHitResult.Location));

				// @todo MeshPaint: Input vector doesn't work well with non-uniform scale
				const float ComponentSpaceBrushRadius = ComponentToWorldMatrix.InverseTransformVector(FVector(BrushProperties->BrushRadius, 0.0f, 0.0f)).Size();
				const float ComponentSpaceSquaredBrushRadius = ComponentSpaceBrushRadius * ComponentSpaceBrushRadius;

				const TArray<FVector>& InRangeVertices = MeshAdapter->SphereIntersectVertices(ComponentSpaceSquaredBrushRadius, ComponentSpaceBrushPosition, ComponentSpaceCameraPosition, VertexProperties->bOnlyFrontFacingTriangles);

				for (const FVector& Vertex : InRangeVertices)
				{
					const FVector WorldPositionVertex = ComponentToWorldMatrix.TransformPosition(Vertex);
					if ((LastBestHitResult.Location - WorldPositionVertex).Size() <= BrushProperties->BrushRadius)
					{
						const float VisualBiasDistance = 0.15f;
						const FVector VertexVisualPosition = WorldPositionVertex + LastBestHitResult.Normal * VisualBiasDistance;
						Draw.DrawPoint(VertexVisualPosition, HoverVertexPointColor, PointDrawSize, SDPG_World);
					}
				}
			}
		}
	}
	else
	{
		BrushStampIndicator->bDrawIndicatorLines = false;
	}
	Draw.EndFrame();
	UpdateResult();

}

void UMeshVertexPaintingTool::OnTick(float DeltaTime)
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		if (MeshPaintingSubsystem->bNeedsRecache)
		{
			CacheSelectionData();
		}
	}

	if (bStampPending)
	{
		Paint(PendingStampRay.Origin, PendingStampRay.Direction);
		bStampPending = false;

		// flow
		if (bInDrag && VertexProperties && VertexProperties->bEnableFlow)
		{
			bStampPending = true;
		}
	}
}

void UMeshVertexPaintingTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);
	bResultValid = false;
}


double UMeshVertexPaintingTool::EstimateMaximumTargetDimension()
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		FBoxSphereBounds::Builder ExtentsBuilder;
		for (UMeshComponent* SelectedComponent : MeshPaintingSubsystem->GetSelectedMeshComponents())
		{
			ExtentsBuilder += SelectedComponent->Bounds;
		}

		if (ExtentsBuilder.IsValid())
		{
			return FBoxSphereBounds(ExtentsBuilder).BoxExtent.GetAbsMax();
		}
	}

	return Super::EstimateMaximumTargetDimension();
	
}

double UMeshVertexPaintingTool::CalculateTargetEdgeLength(int TargetTriCount)
{
	double TargetTriArea = InitialMeshArea / (double)TargetTriCount;
	double EdgeLen = (TargetTriArea);
	return (double)FMath::RoundToInt(EdgeLen*100.0) / 100.0;
}

bool UMeshVertexPaintingTool::Paint(const FVector& InRayOrigin, const FVector& InRayDirection)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintModeAction PaintAction = GetShiftToggle() ? EMeshPaintModeAction::Erase : EMeshPaintModeAction::Paint;
	const float PaintStrength = 1.0f; //Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	TPair<FVector, FVector> Ray(InRayOrigin, InRayDirection);
	return PaintInternal(MakeArrayView(&Ray, 1), PaintAction, PaintStrength);
}

bool UMeshVertexPaintingTool::Paint(const TArrayView<TPair<FVector, FVector>>& Rays)
{
	// Determine paint action according to whether or not shift is held down
	const EMeshPaintModeAction PaintAction = GetShiftToggle() ? EMeshPaintModeAction::Erase : EMeshPaintModeAction::Paint;

	const float PaintStrength = 1.0f; //Viewport->IsPenActive() ? Viewport->GetTabletPressure() : 1.f;
	// Handle internal painting functionality
	return PaintInternal(Rays, PaintAction, PaintStrength);
}


bool UMeshVertexPaintingTool::PaintInternal(const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintModeAction PaintAction, float PaintStrength)
{
	TArray<FPaintRayResults> PaintRayResults;
	PaintRayResults.AddDefaulted(Rays.Num());
	LastBestHitResult.Reset();
	TMap<UMeshComponent*, TArray<int32>> HoveredComponents;
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	const float BrushRadius = BrushProperties->BrushRadius;
	const bool bIsPainting = PaintAction == EMeshPaintModeAction::Paint;
	const float InStrengthScale = PaintStrength;

	bool bPaintApplied = false;
	// Fire out a ray to see if there is a *selected* component under the mouse cursor that can be painted.
	for (int32 i = 0; i < Rays.Num(); ++i)
	{
		const FVector& RayOrigin = Rays[i].Key;
		const FVector& RayDirection = Rays[i].Value;
		FRay Ray = FRay(RayOrigin, RayDirection);
		FHitResult& BestTraceResult = PaintRayResults[i].BestTraceResult;
		const FVector TraceStart(RayOrigin);
		const FVector TraceEnd(RayOrigin + RayDirection * HALF_WORLD_MAX);

		for (UMeshComponent* MeshComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
		{
			TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(MeshComponent);

			// Ray trace
			FHitResult TraceHitResult(1.0f);

			if (MeshAdapter->LineTraceComponent(TraceHitResult, TraceStart, TraceEnd, FCollisionQueryParams(SCENE_QUERY_STAT(Paint), true)))
			{
				// Find the closest impact
				if ((BestTraceResult.GetComponent() == nullptr) || (TraceHitResult.Time < BestTraceResult.Time))
				{
					BestTraceResult = TraceHitResult;
				}
			}
		}

		bool bUsed = false;

		if (BestTraceResult.GetComponent() != nullptr)
		{
			FBox BrushBounds = FBox::BuildAABB(BestTraceResult.Location, FVector(BrushRadius * 1.25f, BrushRadius * 1.25f, BrushRadius * 1.25f));

			// Vertex paint mode, so we want all valid components overlapping the brush hit location
			for (auto TestComponent : MeshPaintingSubsystem->GetPaintableMeshComponents())
			{
				const FBox ComponentBounds = TestComponent->Bounds.GetBox();
				TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(TestComponent);
				if (MeshAdapter && ComponentBounds.Intersect(BrushBounds))
				{
					// OK, this mesh potentially overlaps the brush!
					HoveredComponents.FindOrAdd(TestComponent).Add(i);
					bUsed = true;
				}
			}
		}
		if (bUsed)
		{
			FVector BrushXAxis, BrushYAxis;
			BestTraceResult.Normal.FindBestAxisVectors(BrushXAxis, BrushYAxis);
			// Display settings
			const float VisualBiasDistance = 0.15f;
			const FVector BrushVisualPosition = BestTraceResult.Location + BestTraceResult.Normal * VisualBiasDistance;

			const FLinearColor PaintColor = VertexProperties->PaintColor;
			const FLinearColor EraseColor = VertexProperties->EraseColor;

			// NOTE: We square the brush strength to maximize slider precision in the low range
			const float BrushStrength = BrushProperties->BrushStrength *  BrushProperties->BrushStrength * InStrengthScale;

			const float BrushDepth = BrushRadius;
			LastBestHitResult = BestTraceResult;
			// Mesh paint settings
			FMeshPaintParameters& Params = PaintRayResults[i].Params;
			Params.PaintAction = PaintAction;
			Params.BrushPosition = BestTraceResult.Location;
			Params.BrushNormal = BestTraceResult.Normal;
			Params.BrushColor = bIsPainting ? PaintColor : EraseColor;
			Params.SquaredBrushRadius = BrushRadius * BrushRadius;
			Params.BrushRadialFalloffRange = BrushProperties->BrushFalloffAmount * BrushRadius;
			Params.InnerBrushRadius = BrushRadius - Params.BrushRadialFalloffRange;
			Params.BrushDepth = BrushDepth;
			Params.BrushDepthFalloffRange = BrushProperties->BrushFalloffAmount * BrushDepth;
			Params.BrushDepthFalloffRange = BrushProperties->BrushFalloffAmount * BrushDepth;
			Params.InnerBrushDepth = BrushDepth - Params.BrushDepthFalloffRange;
			Params.BrushStrength = BrushStrength;
			Params.BrushToWorldMatrix = FMatrix(BrushXAxis, BrushYAxis, Params.BrushNormal, Params.BrushPosition);
			Params.InverseBrushToWorldMatrix = Params.BrushToWorldMatrix.Inverse();

			SetAdditionalPaintParameters(Params);
		
		}
	}

	if (HoveredComponents.Num() > 0)
	{
		if (bArePainting == false)
		{
				// Vertex painting is an ongoing transaction, while texture painting is handled separately later in a single transaction
				GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshPaintMode_VertexPaint_TransactionPaintStroke", "Vertex Paint"));
				bArePainting = true;
		}

		// Iterate over the selected meshes under the cursor and paint them!
		for (auto& Entry : HoveredComponents)
		{
			UMeshComponent* HoveredComponent = Entry.Key;
			TArray<int32>& PaintRayResultIds = Entry.Value;

			IMeshPaintComponentAdapter* MeshAdapter = MeshPaintingSubsystem->GetAdapterForComponent(HoveredComponent).Get();
			if (!ensure(MeshAdapter))
			{
				continue;
			}

			if (MeshAdapter->SupportsVertexPaint())
			{
				FPerVertexPaintActionArgs Args;
				Args.Adapter = MeshAdapter;
				FViewCameraState CameraState;
				GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
				Args.CameraPosition = CameraState.Position;
				Args.BrushProperties = VertexProperties;
				Args.Action = PaintAction;

				bool bMeshPreEditCalled = false;

				TSet<int32> InfluencedVertices;
				for (int32 PaintRayResultId : PaintRayResultIds)
				{
					InfluencedVertices.Reset();
					Args.HitResult = PaintRayResults[PaintRayResultId].BestTraceResult;
					bPaintApplied |= MeshPaintingSubsystem->GetPerVertexPaintInfluencedVertices(Args, InfluencedVertices);

					if (InfluencedVertices.Num() == 0)
					{
						continue;
					}

					if (!bMeshPreEditCalled)
					{
						bMeshPreEditCalled = true;
						MeshAdapter->PreEdit();
					}

					for (const int32 VertexIndex : InfluencedVertices)
					{
						ApplyVertexData(Args, VertexIndex, PaintRayResults[PaintRayResultId].Params);
					}
				}

				if (bMeshPreEditCalled)
				{
					MeshAdapter->PostEdit();
				}
			}
		}
	}

	return bPaintApplied;
}

void UMeshVertexPaintingTool::ApplyVertexData(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMeshPaintParameters Parameters)
{
	/** Retrieve vertex position and color for applying vertex painting */
	FColor PaintColor;
	FVector Position;
	InArgs.Adapter->GetVertexPosition(VertexIndex, Position);
	Position = InArgs.Adapter->GetComponentToWorldMatrix().TransformPosition(Position);
	InArgs.Adapter->GetVertexColor(VertexIndex, PaintColor, true);
	GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->PaintVertex(Position, Parameters, PaintColor);
	InArgs.Adapter->SetVertexColor(VertexIndex, PaintColor, true);
}


void UMeshVertexPaintingTool::UpdateResult()
{
	GetToolManager()->PostInvalidation();

	bResultValid = true;
}

FInputRayHit UMeshVertexPaintingTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	bCachedClickRay = false;
	if (!HitTest(PressPos.WorldRay, OutHit))
	{
		UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
		const bool bFallbackClick = MeshPaintingSubsystem->GetSelectedMeshComponents().Num() > 0;
		if (SelectionMechanic->IsHitByClick(PressPos, bFallbackClick).bHit)
		{
			bCachedClickRay = true;
			PendingClickRay = PressPos.WorldRay;
			PendingClickScreenPosition = PressPos.ScreenPosition;
			return FInputRayHit(0.0);
		}
	}
	return Super::CanBeginClickDragSequence(PressPos);
}

void UMeshVertexPaintingTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	Super::OnUpdateModifierState(ModifierID, bIsOn);
	SelectionMechanic->SetAddToSelectionSet(bShiftToggle);
}


void UMeshVertexPaintingTool::OnBeginDrag(const FRay& Ray)
{
	Super::OnBeginDrag(Ray);
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		bInDrag = true;

		// apply initial stamp
		PendingStampRay = Ray;
		bStampPending = true;
	}
	else if (bCachedClickRay)
	{
		FInputDeviceRay InputDeviceRay = FInputDeviceRay(PendingClickRay, PendingClickScreenPosition);
		SelectionMechanic->SetAddToSelectionSet(bShiftToggle);
		SelectionMechanic->OnClicked(InputDeviceRay);
		bCachedClickRay = false;
		RecalculateBrushRadius();
	}
}

void UMeshVertexPaintingTool::OnUpdateDrag(const FRay& Ray)
{
	Super::OnUpdateDrag(Ray);
	if (bInDrag)
	{
		PendingStampRay = Ray;
		bStampPending = true;
	}
}



void UMeshVertexPaintingTool::OnEndDrag(const FRay& Ray)
{
	FinishPainting();
	bStampPending = false;
	bInDrag = false;
}

bool UMeshVertexPaintingTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	bool bUsed = false;
	UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>();
	if (MeshPaintingSubsystem)
	{
		MeshPaintingSubsystem->FindHitResult(Ray, OutHit);
		LastBestHitResult = OutHit;
		bUsed = OutHit.bBlockingHit;
	}
	return bUsed;
}

void UMeshVertexPaintingTool::FinishPainting()
{
	if (bArePainting)
	{
		bArePainting = false;
		GetToolManager()->EndUndoTransaction();
		OnPaintingFinishedDelegate.ExecuteIfBound();
	}
}


UMeshVertexColorPaintingTool::UMeshVertexColorPaintingTool()
{
	PropertyClass = UMeshVertexColorPaintingToolProperties::StaticClass();
}

void UMeshVertexColorPaintingTool::Setup()
{
	Super::Setup();
	ColorProperties = Cast<UMeshVertexColorPaintingToolProperties>(BrushProperties);

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartColorPaintTool", "Paint vertex colors on selected meshes.  Use the Color View Mode to preview your applied changes."),
		EToolMessageLevel::UserNotification);
}

void UMeshVertexPaintingTool::CacheSelectionData()
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		MeshPaintingSubsystem->ClearPaintableMeshComponents();
		// Update(cached) Paint LOD level if necessary
		VertexProperties->LODIndex = FMath::Min<int32>(VertexProperties->LODIndex, GetMaxLODIndexToPaint());
		CachedLODIndex = VertexProperties->LODIndex;
		bCachedForceLOD = VertexProperties->bPaintOnSpecificLOD;
		//Determine LOD level to use for painting(can only paint on LODs in vertex mode)
		const int32 PaintLODIndex = VertexProperties->bPaintOnSpecificLOD ? VertexProperties->LODIndex : 0;
		//Determine UV channel to use while painting textures
		const int32 UVChannel = 0;

		MeshPaintingSubsystem->CacheSelectionData(PaintLODIndex, UVChannel);
	}
}

void UMeshVertexColorPaintingTool::SetAdditionalPaintParameters(FMeshPaintParameters& InPaintParameters)
{
	InPaintParameters.bWriteRed = ColorProperties->bWriteRed;
	InPaintParameters.bWriteGreen = ColorProperties->bWriteGreen;
	InPaintParameters.bWriteBlue = ColorProperties->bWriteBlue;
	InPaintParameters.bWriteAlpha = ColorProperties->bWriteAlpha;
	InPaintParameters.ApplyVertexDataDelegate.AddUObject(GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>(), &UMeshPaintingSubsystem::ApplyVertexColorPaint);
}

int32 UMeshVertexPaintingTool::GetMaxLODIndexToPaint() const
{
	//The maximum LOD we can paint is decide by the lowest number of LOD in the selection
	int32 LODMin = TNumericLimits<int32>::Max();
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		TArray<UMeshComponent*> SelectedComponents = MeshPaintingSubsystem->GetSelectedMeshComponents();

		for (UMeshComponent* MeshComponent : SelectedComponents)
		{
			int32 NumMeshLODs = 0;
			if (MeshPaintingSubsystem->TryGetNumberOfLODs(MeshComponent, NumMeshLODs))
			{
				ensure(NumMeshLODs > 0);
				LODMin = FMath::Min(LODMin, NumMeshLODs - 1);
			}
		}
		if (LODMin == TNumericLimits<int32>::Max())
		{
			LODMin = 1;
		}
	}
	return LODMin;
}


void UMeshVertexPaintingTool::LODPaintStateChanged(const bool bLODPaintingEnabled)
{
	bool AbortChange = false;

	// Set actual flag in the settings struct
	VertexProperties->bPaintOnSpecificLOD = bLODPaintingEnabled;

	if (!bLODPaintingEnabled)
	{
		// Reset painting LOD index
		VertexProperties->LODIndex = 0;
	}

	ApplyForcedLODIndex(bLODPaintingEnabled ? CachedLODIndex : -1);

	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();

		TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
		//Make sure all static mesh render is dirty since we change the force LOD
		for (UMeshComponent* SelectedComponent : PaintableComponents)
		{
			if (SelectedComponent)
			{
				ComponentReregisterContext.Reset(new FComponentReregisterContext(SelectedComponent));
			}
		}

		MeshPaintingSubsystem->Refresh();
	}
}

void UMeshVertexPaintingTool::ApplyForcedLODIndex(int32 ForcedLODIndex)
{
	if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
	{
		TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();

		for (UMeshComponent* SelectedComponent : PaintableComponents)
		{
			if (SelectedComponent)
			{
				MeshPaintingSubsystem->ForceRenderMeshLOD(SelectedComponent, ForcedLODIndex);
			}
		}
	}
}


void UMeshVertexPaintingTool::PaintLODChanged()
{
	// Enforced LOD for painting
	if (CachedLODIndex != VertexProperties->LODIndex)
	{
		CachedLODIndex = VertexProperties->LODIndex;
		ApplyForcedLODIndex(bCachedForceLOD ? CachedLODIndex : -1);

		TUniquePtr< FComponentReregisterContext > ComponentReregisterContext;
		//Make sure all static mesh render is dirty since we change the force LOD
		if (UMeshPaintingSubsystem* MeshPaintingSubsystem = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>())
		{
			TArray<UMeshComponent*> PaintableComponents = MeshPaintingSubsystem->GetPaintableMeshComponents();

			for (UMeshComponent* SelectedComponent : PaintableComponents)
			{
				if (SelectedComponent)
				{
					ComponentReregisterContext.Reset(new FComponentReregisterContext(SelectedComponent));
				}
			}

			MeshPaintingSubsystem->Refresh();
		}
	}
}

void UMeshVertexPaintingTool::CycleMeshLODs(int32 Direction)
{
	if (bCachedForceLOD)
	{
		const int32 MaxLODIndex = GetMaxLODIndexToPaint() + 1;
		const int32 NewLODIndex = VertexProperties->LODIndex + Direction;
		const int32 AdjustedLODIndex = NewLODIndex < 0 ? MaxLODIndex + NewLODIndex : NewLODIndex % MaxLODIndex;
		VertexProperties->LODIndex = AdjustedLODIndex;
		PaintLODChanged();
	}
}

UMeshVertexWeightPaintingToolProperties::UMeshVertexWeightPaintingToolProperties()
	:UMeshVertexPaintingToolProperties(),
	TextureWeightType(EMeshPaintWeightTypes::AlphaLerp),
	PaintTextureWeightIndex(EMeshPaintTextureIndex::TextureOne),
	EraseTextureWeightIndex(EMeshPaintTextureIndex::TextureTwo)
{

}

UMeshVertexWeightPaintingTool::UMeshVertexWeightPaintingTool()
{
	PropertyClass = UMeshVertexWeightPaintingToolProperties::StaticClass();
}

void UMeshVertexWeightPaintingTool::Setup()
{
	Super::Setup();
	WeightProperties = Cast<UMeshVertexWeightPaintingToolProperties>(BrushProperties);

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartPaintWeightsTool", "Paint Vertex Weights on selected meshes."),
		EToolMessageLevel::UserNotification);
}

void UMeshVertexWeightPaintingTool::SetAdditionalPaintParameters(FMeshPaintParameters& InPaintParameters)
{
	InPaintParameters.TotalWeightCount = (int32)WeightProperties->TextureWeightType;

	// Select texture weight index based on whether or not we're painting or erasing
	{
		const int32 PaintWeightIndex = InPaintParameters.PaintAction == EMeshPaintModeAction::Paint ? (int32)WeightProperties->PaintTextureWeightIndex : (int32)WeightProperties->EraseTextureWeightIndex;

		// Clamp the weight index to fall within the total weight count
		InPaintParameters.PaintWeightIndex = FMath::Clamp(PaintWeightIndex, 0, InPaintParameters.TotalWeightCount - 1);
	}
	InPaintParameters.ApplyVertexDataDelegate.AddUObject(GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>(), &UMeshPaintingSubsystem::ApplyVertexWeightPaint);
}

#undef LOCTEXT_NAMESPACE

