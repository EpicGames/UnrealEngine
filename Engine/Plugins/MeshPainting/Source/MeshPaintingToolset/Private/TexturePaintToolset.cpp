// Copyright Epic Games, Inc. All Rights Reserved.

#include "TexturePaintToolset.h"

#include "Editor.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include "IMeshPaintComponentAdapter.h"

#include "Engine/TextureRenderTarget2D.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "MaterialShared.h"
#include "MeshPaintingToolsetTypes.h"
#include "RenderingThread.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TexturePaintToolset)

void UTexturePaintToolset::CopyTextureToRenderTargetTexture(UTexture* SourceTexture, UTextureRenderTarget2D* RenderTargetTexture, ERHIFeatureLevel::Type FeatureLevel)
{
	check(SourceTexture != nullptr);
	check(RenderTargetTexture != nullptr);

	// Grab the actual render target resource from the texture.  Note that we're absolutely NOT ALLOWED to
	// dereference this pointer.  We're just passing it along to other functions that will use it on the render
	// thread.  The only thing we're allowed to do is check to see if it's nullptr or not.
	FTextureRenderTargetResource* RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();
	check(RenderTargetResource != nullptr);
	
	// Create a canvas for the render target and clear it to black
	FCanvas Canvas(RenderTargetResource, nullptr, FGameTime(), FeatureLevel);

	const uint32 Width = RenderTargetTexture->GetSurfaceWidth();
	const uint32 Height = RenderTargetTexture->GetSurfaceHeight();

	// @todo MeshPaint: Need full color/alpha writes enabled to get alpha
	// @todo MeshPaint: Texels need to line up perfectly to avoid bilinear artifacts
	// @todo MeshPaint: Potential gamma issues here
	// @todo MeshPaint: Probably using CLAMP address mode when reading from source (if texels line up, shouldn't matter though.)

	// @todo MeshPaint: Should use scratch texture built from original source art (when possible!)
	//		-> Current method will have compression artifacts!
	
	// Grab the texture resource.  We only support 2D textures and render target textures here.
	FTexture* TextureResource = nullptr;
	UTexture2D* Texture2D = Cast<UTexture2D>(SourceTexture);
	if (Texture2D != nullptr)
	{
		TextureResource = Texture2D->GetResource();
	}
	else
	{
		UTextureRenderTarget2D* TextureRenderTarget2D = Cast<UTextureRenderTarget2D>(SourceTexture);
		TextureResource = TextureRenderTarget2D->GameThread_GetRenderTargetResource();
	}
	check(TextureResource != nullptr);
	
	// Draw a quad to copy the texture over to the render target
	{
		const float MinU = 0.0f;
		const float MinV = 0.0f;
		const float MaxU = 1.0f;
		const float MaxV = 1.0f;
		const float MinX = 0.0f;
		const float MinY = 0.0f;
		const float MaxX = Width;
		const float MaxY = Height;

		FCanvasUVTri Tri1;
		FCanvasUVTri Tri2;
		Tri1.V0_Pos = FVector2D(MinX, MinY);
		Tri1.V0_UV = FVector2D(MinU, MinV);
		Tri1.V1_Pos = FVector2D(MaxX, MinY);
		Tri1.V1_UV = FVector2D(MaxU, MinV);
		Tri1.V2_Pos = FVector2D(MaxX, MaxY);
		Tri1.V2_UV = FVector2D(MaxU, MaxV);

		Tri2.V0_Pos = FVector2D(MaxX, MaxY);
		Tri2.V0_UV = FVector2D(MaxU, MaxV);
		Tri2.V1_Pos = FVector2D(MinX, MaxY);
		Tri2.V1_UV = FVector2D(MinU, MaxV);
		Tri2.V2_Pos = FVector2D(MinX, MinY);
		Tri2.V2_UV = FVector2D(MinU, MinV);
		Tri1.V0_Color = Tri1.V1_Color = Tri1.V2_Color = Tri2.V0_Color = Tri2.V1_Color = Tri2.V2_Color = FLinearColor::White;
		TArray< FCanvasUVTri > List;
		List.Add(Tri1);
		List.Add(Tri2);
		FCanvasTriangleItem TriItem(List, TextureResource);
		TriItem.BlendMode = SE_BLEND_Opaque;
		Canvas.DrawItem(TriItem);
	}

	// Tell the rendering thread to draw any remaining batched elements
	Canvas.Flush_GameThread(true);
	
	ENQUEUE_RENDER_COMMAND(UpdateMeshPaintRTCommand)(
		[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
		{
			TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
		});
}

bool UTexturePaintToolset::GenerateSeamMask(UMeshComponent* MeshComponent, int32 UVSet, UTextureRenderTarget2D* SeamRenderTexture, UTexture2D* Texture, UTextureRenderTarget2D* RenderTargetTexture)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent);
	if (StaticMeshComponent == nullptr)
	{
		return false;
	}

	const int32 PaintingMeshLODIndex = 0;

	check(StaticMeshComponent != nullptr);
	check(StaticMeshComponent->GetStaticMesh() != nullptr);
	check(SeamRenderTexture != nullptr);
	check(StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[PaintingMeshLODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() > (uint32)UVSet);

	bool RetVal = false;

	FStaticMeshLODResources& LODModel = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[PaintingMeshLODIndex];

	const uint32 Width = SeamRenderTexture->GetSurfaceWidth();
	const uint32 Height = SeamRenderTexture->GetSurfaceHeight();

	// Grab the actual render target resource from the texture.  Note that we're absolutely NOT ALLOWED to
	// dereference this pointer.  We're just passing it along to other functions that will use it on the render
	// thread.  The only thing we're allowed to do is check to see if it's nullptr or not.
	FTextureRenderTargetResource* RenderTargetResource = SeamRenderTexture->GameThread_GetRenderTargetResource();
	check(RenderTargetResource != nullptr);

	const bool bIsMeshPaintTexture = MeshComponent->GetMeshPaintTexture() == Texture;

	const int32 NumElements = StaticMeshComponent->GetNumMaterials();
	UTexture2D* TargetTexture2D = Texture;

	// Store info that tells us if the element material uses our target texture.
	// We will use this info to eliminate triangles that do not use our texture.
	TArray<bool> ElementUsesTargetTexture;
	bool bAnyElementUsesTargetTexture = false;
	ElementUsesTargetTexture.AddZeroed(NumElements);
	for (int32 ElementIndex = 0; ElementIndex < NumElements; ElementIndex++)
	{
		ElementUsesTargetTexture[ElementIndex] = false;

		UMaterialInterface* ElementMat = StaticMeshComponent->GetMaterial(ElementIndex);
		if (ElementMat != nullptr)
		{
			if (bIsMeshPaintTexture)
			{
				ElementUsesTargetTexture[ElementIndex] |= ElementMat->HasMeshPaintTexture();
			}
			else
			{
				ElementUsesTargetTexture[ElementIndex] |= DoesMaterialUseTexture(ElementMat, TargetTexture2D);

				if (ElementUsesTargetTexture[ElementIndex] == false && RenderTargetTexture != nullptr)
				{
					// If we didn't get a match on our selected texture, we'll check to see if the the material uses a
					//  render target texture override that we put on during painting.
					ElementUsesTargetTexture[ElementIndex] |= DoesMaterialUseTexture(ElementMat, RenderTargetTexture);
				}
			}
		}

		// We track if there is no section that uses the texture.
		// That would be a special case where we are painting without any context for the seams. Then seam painting would expand/blur all painting.
		// To avoid that it's better to render _all_ sections into the seam mask rather than none.
		bAnyElementUsesTargetTexture |= ElementUsesTargetTexture[ElementIndex];
	}

	{
		// Create a canvas for the render target and clear it to white
		FCanvas Canvas(RenderTargetResource, nullptr, FGameTime(), GEditor->GetEditorWorldContext().World()->GetFeatureLevel());
		Canvas.Clear(FLinearColor::White);

		FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
		
		TArray<FCanvasUVTri> TriList;
		for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
		{
			FStaticMeshSection& Element = LODModel.Sections[ElementIndex];

			if (ElementUsesTargetTexture[Element.MaterialIndex] || !bAnyElementUsesTargetTexture)
			{
				for (uint32 TriIndex = Element.FirstIndex / 3u; TriIndex < Element.FirstIndex / 3u + Element.NumTriangles; ++TriIndex)
				{
					// Grab the vertex indices and points for this triangle
					FVector2D TriUVs[3];
					FVector2D UVMin(99999.9f, 99999.9f);
					FVector2D UVMax(-99999.9f, -99999.9f);
					for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
					{
						const int32 VertexIndex = Indices[TriIndex * 3 + TriVertexNum];
						TriUVs[TriVertexNum] = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVSet));

						// Update bounds
						float U = TriUVs[TriVertexNum].X;
						float V = TriUVs[TriVertexNum].Y;

						if (U < UVMin.X)
						{
							UVMin.X = U;
						}
						if (U > UVMax.X)
						{
							UVMax.X = U;
						}
						if (V < UVMin.Y)
						{
							UVMin.Y = V;
						}
						if (V > UVMax.Y)
						{
							UVMax.Y = V;
						}
					}

					// If the triangle lies entirely outside of the 0.0-1.0 range, we'll transpose it back
					FVector2D UVOffset(0.0f, 0.0f);
					if (UVMax.X > 1.0f)
					{
						UVOffset.X = -FMath::FloorToInt(UVMin.X);
					}
					else if (UVMin.X < 0.0f)
					{
						UVOffset.X = 1.0f + FMath::FloorToInt(-UVMax.X);
					}

					if (UVMax.Y > 1.0f)
					{
						UVOffset.Y = -FMath::FloorToInt(UVMin.Y);
					}
					else if (UVMin.Y < 0.0f)
					{
						UVOffset.Y = 1.0f + FMath::FloorToInt(-UVMax.Y);
					}

					// Note that we "wrap" the texture coordinates here to handle the case where the user
					// is painting on a tiling texture, or with the UVs out of bounds.  Ideally all of the
					// UVs would be in the 0.0 - 1.0 range but sometimes content isn't setup that way.
					// @todo MeshPaint: Handle triangles that cross the 0.0-1.0 UV boundary?
					FVector2D TrianglePoints[3];
					for (int32 TriVertexNum = 0; TriVertexNum < 3; ++TriVertexNum)
					{
						TriUVs[TriVertexNum].X += UVOffset.X;
						TriUVs[TriVertexNum].Y += UVOffset.Y;

						TrianglePoints[TriVertexNum].X = TriUVs[TriVertexNum].X * Width;
						TrianglePoints[TriVertexNum].Y = TriUVs[TriVertexNum].Y * Height;
					}

					FCanvasUVTri& Tri = TriList.Emplace_GetRef();
					Tri.V0_Pos = TrianglePoints[0];
					Tri.V0_UV = TriUVs[0];
					Tri.V0_Color = FLinearColor::Black;
					Tri.V1_Pos = TrianglePoints[1];
					Tri.V1_UV = TriUVs[1];
					Tri.V1_Color = FLinearColor::Black;
					Tri.V2_Pos = TrianglePoints[2];
					Tri.V2_UV = TriUVs[2];
					Tri.V2_Color = FLinearColor::Black;
				}
			}
		}

		if (TriList.Num())
		{
			// Setup the tri render item with the list of tris
			FCanvasTriangleItem TriItem(TriList, RenderTargetResource);
			TriItem.BlendMode = SE_BLEND_Opaque;
			// And render it
			Canvas.DrawItem(TriItem);
			// Tell the rendering thread to draw any remaining batched elements
			Canvas.Flush_GameThread(true);
		}
	}

	ENQUEUE_RENDER_COMMAND(UpdateMeshPaintRTCommand5)(
		[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
	{
		TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
	});

	return RetVal;
}

int32 UTexturePaintToolset::GetMaxSupportedBytesPerPixelForPainting()
{
	return GPixelFormats[GetTempUncompressedTexturePixelFormat()].BlockBytes;
}

EPixelFormat UTexturePaintToolset::GetTempUncompressedTexturePixelFormat()
{
	return EPixelFormat::PF_B8G8R8A8;
}

UTexture2D* UTexturePaintToolset::CreateScratchUncompressedTexture(UTexture2D* SourceTexture)
{
	check(SourceTexture->Source.IsValid());

	// Decompress PNG image and convert to BGRA8 for painting.
	FImage SourceImage;
	SourceTexture->Source.GetMipImage(SourceImage, 0);
	SourceImage.ChangeFormat(ERawImageFormat::BGRA8, SourceImage.GetGammaSpace());

	// Allocate the new texture
	UTexture2D* NewTexture2D = NewObject<UTexture2D>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional);

	// Fill in the base mip for the texture we created
	NewTexture2D->Source.Init(SourceImage);

	// Set options
	NewTexture2D->SRGB = SourceTexture->SRGB;
	NewTexture2D->CompressionNone = true;
	NewTexture2D->MipGenSettings = TMGS_NoMipmaps;
	NewTexture2D->CompressionSettings = TC_Default;

	// Update the remote texture data
	NewTexture2D->UpdateResource();
	return NewTexture2D;
}

// Keep old legacy method of initializing render target data for the paint brush texture; @todo MeshPaint: Migrate to the method with texture re-use
void UTexturePaintToolset::SetupInitialRenderTargetData(UTexture2D* InTextureSource, UTextureRenderTarget2D* InRenderTarget)
{
	check(InTextureSource != nullptr);
	check(InRenderTarget != nullptr);

	if (InTextureSource->Source.IsValid())
	{
		// Great, we have source data!  We'll use that as our image source.

		// Create a texture in memory from the source art
		{
			// @todo MeshPaint: This generates a lot of memory thrash -- try to cache this texture and reuse it?
			UTexture2D* TempSourceArtTexture = CreateScratchUncompressedTexture(InTextureSource);
			check(TempSourceArtTexture != nullptr);

#if WITH_EDITOR
			// We need to complete texture compilation before we can copy to render target.
			TempSourceArtTexture->BlockOnAnyAsyncBuild();
#endif

			// Copy the texture to the render target using the GPU
			CopyTextureToRenderTargetTexture(TempSourceArtTexture, InRenderTarget, GEditor->GetEditorWorldContext().World()->GetFeatureLevel());

			// NOTE: TempSourceArtTexture is no longer needed (will be GC'd)
		}
	}
	else
	{
		// Just copy (render) the texture in GPU memory to our render target.  Hopefully it's not
		// compressed already!
		check(InTextureSource->IsFullyStreamedIn());
		CopyTextureToRenderTargetTexture(InTextureSource, InRenderTarget, GEditor->GetEditorWorldContext().World()->GetFeatureLevel());
	}
}

void UTexturePaintToolset::FindMaterialIndicesUsingTexture(const UTexture* Texture, const UMeshComponent* MeshComponent, TArray<int32>& OutIndices)
{
	checkf(Texture && MeshComponent, TEXT("Invalid Texture of MeshComponent"));

	const bool bIsMeshPaintTexture = MeshComponent->GetMeshPaintTexture() == Texture;

	const int32 NumMaterials = MeshComponent->GetNumMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
	{
		const UMaterialInterface* MaterialInterface = MeshComponent->GetMaterial(MaterialIndex);
		if (MaterialInterface)
		{
			bool bUsesTexture = false;
			if (bIsMeshPaintTexture)
			{
				bUsesTexture = MaterialInterface->HasMeshPaintTexture();
			}
			else
			{
				bUsesTexture = DoesMaterialUseTexture(MaterialInterface, Texture);
			}
			if (bUsesTexture)
			{
				OutIndices.AddUnique(MaterialIndex);
			}
		}
	}
}

void UTexturePaintToolset::RetrieveMeshSectionsForTextures(const UMeshComponent* MeshComponent, int32 LODIndex, TArray<const UTexture*> Textures, TArray<FTexturePaintMeshSectionInfo>& OutSectionInfo)
{	
	// @todo MeshPaint: if LODs can use different materials/textures then this will cause us problems
	TArray<int32> MaterialIndices;
	for (const UTexture* Texture : Textures)
	{
		UTexturePaintToolset::FindMaterialIndicesUsingTexture(Texture, MeshComponent, MaterialIndices);
	}
	
	if (MaterialIndices.Num())
	{
		UTexturePaintToolset::RetrieveMeshSectionsForMaterialIndices(MeshComponent, LODIndex, MaterialIndices, OutSectionInfo);
	}
}

void UTexturePaintToolset::RetrieveMeshSectionsForMaterialIndices(const UMeshComponent* MeshComponent, int32 LODIndex, const TArray<int32>& MaterialIndices, TArray<FTexturePaintMeshSectionInfo>& OutSectionInfo)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(MeshComponent))
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

		if (StaticMesh)
		{
			//@TODO: Find a better way to move this generically to the adapter
			check(StaticMeshComponent->GetStaticMesh()->GetNumLODs() > (int32)LODIndex);
			const FStaticMeshLODResources& LODModel = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];
			const int32 NumSections = LODModel.Sections.Num();

			FTexturePaintMeshSectionInfo Info;
			for ( const FStaticMeshSection& Section : LODModel.Sections)
			{
				const bool bSectionUsesTexture = MaterialIndices.Contains(Section.MaterialIndex);
				if (bSectionUsesTexture)
				{
					Info.FirstIndex = (Section.FirstIndex / 3);
					Info.LastIndex = Info.FirstIndex + Section.NumTriangles;
					OutSectionInfo.Add(Info);
				}
			}
		}
	}
	else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<const USkeletalMeshComponent>(MeshComponent))
	{
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
			const FSkeletalMeshRenderData* Resource = SkeletalMesh->GetResourceForRendering();
			checkf(Resource->LODRenderData.IsValidIndex(LODIndex), TEXT("Invalid index %i for LOD models in Skeletal Mesh"), LODIndex);
			const FSkeletalMeshLODRenderData& LODData = Resource->LODRenderData[LODIndex];
			FTexturePaintMeshSectionInfo Info;
			for (const FSkelMeshRenderSection& Section : LODData.RenderSections)
			{
				Info.FirstIndex = Section.BaseIndex;
				Info.LastIndex = (Section.BaseIndex / 3) + Section.NumTriangles;
				OutSectionInfo.Add(Info);
			}
		}
	}
}

void UTexturePaintToolset::RetrieveTexturesForComponent(const UMeshComponent* Component, IMeshPaintComponentAdapter* Adapter, int32& OutDefaultIndex, TArray<FPaintableTexture>& OutTextures)
{
	OutDefaultIndex = INDEX_NONE;

	if (Component && Adapter)
	{
		// Get the materials used by the mesh
		TArray<UMaterialInterface*> UsedMaterials;
		Component->GetUsedMaterials(UsedMaterials);

		for (int32 MaterialIndex = 0; MaterialIndex < UsedMaterials.Num(); ++MaterialIndex)
		{
			int32 OutDefaultIndexForMaterial = INDEX_NONE;
			Adapter->QueryPaintableTextures(MaterialIndex, OutDefaultIndexForMaterial, OutTextures);

			// We can only collect one default texture from the multiple materials!
			if (OutDefaultIndex == INDEX_NONE && OutDefaultIndexForMaterial != INDEX_NONE)
			{
				OutDefaultIndex = OutDefaultIndexForMaterial;
			}
		}
	}
}
