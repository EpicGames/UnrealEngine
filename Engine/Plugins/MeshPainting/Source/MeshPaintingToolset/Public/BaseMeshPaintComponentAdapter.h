// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMeshPaintComponentAdapter.h"
#include "TMeshPaintOctree.h"
#include "MeshAdapter.h"
#include "Spatial/MeshAABBTree3.h" // IWYU pragma: keep
#include "UObject/GCObject.h"

#define UE_API MESHPAINTINGTOOLSET_API

typedef UE::Geometry::TIndexMeshArrayAdapter<uint32, double> FIndexMeshArrayAdapterd;

/** Base mesh paint geometry adapter, handles basic sphere intersection using a Octree */
class FBaseMeshPaintComponentAdapter : public IMeshPaintComponentAdapter, public FGCObject, public TSharedFromThis<FBaseMeshPaintComponentAdapter>
{
public:
	/** Start IMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool Initialize() override;
	UE_API virtual const TArray<FVector>& GetMeshVertices() const override;
	UE_API virtual const TArray<uint32>& GetMeshIndices() const override;
	UE_API virtual void GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const override;
	UE_API virtual TArray<uint32> SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;
	UE_API virtual void GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const override;
	UE_API virtual void GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const override;
	UE_API virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const override;
	UE_API virtual bool RayIntersectAdapter(UE::Geometry::FIndex3i& HitTriangle, FVector& HitPosition, const FVector Start, const FVector End) const override;
	/** End IMeshPaintGeometryAdapter Overrides */

	virtual FString GetReferencerName() const override
	{
		return TEXT("FBaseMeshPaintComponentAdapter");
	}

	virtual bool InitializeVertexData() = 0;
protected:
	UE_API bool BuildOctree();
protected:
	/** Index and Vertex data populated by derived classes in InitializeVertexData */
	TArray<FVector> MeshVertices;
	TArray<uint32> MeshIndices;
	/** Octree used for reducing the cost of sphere intersecting with triangles / vertices */
	TUniquePtr<FMeshPaintTriangleOctree> MeshTriOctree;
	FIndexMeshArrayAdapterd Adapter;
	TUniquePtr<UE::Geometry::TMeshAABBTree3<const FIndexMeshArrayAdapterd>> AABBTree;
};

#undef UE_API
