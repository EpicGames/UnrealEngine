// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/BaseBrushTool.h"
#include "BaseMeshPaintingToolProperties.generated.h"

UCLASS(MinimalAPI)
class UMeshPaintingToolProperties : public UBrushBaseProperties
{
	GENERATED_BODY()

public:
	/** Color used for applying color painting */
	UPROPERTY(EditAnywhere, Category = ColorPainting)
	FLinearColor PaintColor = FLinearColor::White;

	/** Color used for erasing color painting */
	UPROPERTY(EditAnywhere, Category = ColorPainting)
	FLinearColor EraseColor = FLinearColor::Transparent;

	/** Enables "Flow" painting where paint is continually applied from the brush every tick */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Enable Brush Flow"))
	bool bEnableFlow = false;

	/** Whether back-facing triangles should be ignored */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Ignore Back-Facing"))
	bool bOnlyFrontFacingTriangles = false;
};
