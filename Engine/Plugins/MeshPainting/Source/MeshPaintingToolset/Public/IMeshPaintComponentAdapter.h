// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Matrix.h"

#define UE_API MESHPAINTINGTOOLSET_API

class FMaterialUpdateContext;
class FReferenceCollector;
class UMeshComponent;
class UTexture;
class UMaterialInterface;

namespace UE
{
	namespace Geometry
	{
		struct FIndex3i;
	}
}

namespace ERHIFeatureLevel { enum Type : int; }

/**
 * Interface for a class to provide mesh painting support for a subclass of UMeshComponent
 */
class IMeshPaintComponentAdapter
{
public:
	/** Constructs the adapter for a specific LOD index using the mesh component */
	virtual bool Construct(UMeshComponent* InComponent, int32 MeshLODIndex) = 0;

	/** D-tor */
	virtual ~IMeshPaintComponentAdapter() = default;
	
	/** Returns whether or not initialization of necessary data was successful */
	virtual bool Initialize() = 0;

	/** Called when this adapter is created and added to its owner (used for setting up) */
	virtual void OnAdded() = 0;
	
	/** Called when this adapter is removed from its owner and deleted (used for cleaning up) */
	virtual void OnRemoved() = 0;

	/** Returns whether or not this adapter is in a valid state */
	virtual bool IsValid() const = 0;	
	
	/** Whether or not this adapter supports texture asset painting */
	virtual bool SupportsTexturePaint() const = 0;

	/** Whether or not this adapter supports texture color painting */
	virtual bool SupportsTextureColorPaint() const = 0;

	/** Whether or not this adapter supports vertex painting */
	virtual bool SupportsVertexPaint() const = 0;

	/** Returns the number of UV channels for the current LOD index in the Mesh. */
	virtual int32 GetNumUVChannels() const = 0;

	/** Returns the result of tracing a line on the component represented by this adapter */
	virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const = 0;	
	
	/** Retrieves a list of textures which are suitable for painting */
	virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) = 0;

	/** Applies or removes an override texture to use while rendering the meshes materials */
	virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const = 0;
	
	UE_DEPRECATED(5.7, "Use the version that takes a FMaterialUpdateContext.")
	void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture) const;

	/** Returns the vertices for the current LOD index in the Mesh */
	virtual const TArray<FVector>& GetMeshVertices() const = 0;

	/** Returns the indices for the current LOD index in the Mesh */
	virtual const TArray<uint32>& GetMeshIndices() const = 0;

	/** Returns the Vertex Position at Vertex Index from the Mesh */
	virtual void GetVertexPosition(int32 VertexIndex, FVector& OutVertex) const = 0;

	/** Sets the Vertex Color at Vertex Index to Color inside of the Mesh */
	virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) = 0;
	
	/** Returns the Texture Coordinate at Vertex Index from the Mesh */
	virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const = 0;

	/** Returns the Vertex Color at Vertex Index from the Mesh */
	virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const = 0;

	/** Returns the Component to World matrix from the Mesh Component */
	virtual FMatrix GetComponentToWorldMatrix() const = 0;

	/** Pre Edit to setup necessary data */
	virtual void PreEdit() = 0;

	/** Post Edit to clear up or update necessary data */
	virtual void PostEdit() = 0;

	/** Default functionality for querying textures from a mesh component which are suitable for texture painting */
	static UE_API void DefaultQueryPaintableTextures(int32 MaterialIndex, const UMeshComponent* MeshComponent, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList);

	/** Returns the triangle indices which intersect with the given sphere */
	virtual TArray<uint32> SphereIntersectTriangles(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const = 0;

	/** Retrieves the influenced vertex indices which intersect the given sphere */
	virtual void GetInfluencedVertexIndices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TSet<int32> &InfluencedVertices) const = 0;
	
	/** Retrieves the influenced vertex indices and positions that intersect the given sphere */
	virtual void GetInfluencedVertexData(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing, TArray<TPair<int32, FVector>>& OutData) const = 0;

	/** Returns the vertex positions which intersect the given sphere */
	virtual TArray<FVector> SphereIntersectVertices(const float ComponentSpaceSquaredBrushRadius, const FVector& ComponentSpaceBrushPosition, const FVector& ComponentSpaceCameraPosition, const bool bOnlyFrontFacing) const = 0;

	/** Returns the hit result which intersect with the given ray */
	virtual bool RayIntersectAdapter(UE::Geometry::FIndex3i& HitTriangle, FVector& HitPosition, const FVector Start, const FVector End) const = 0;

};

namespace UE::MeshPaintingToolset
{
	/*
	 * Provide a default implementation for the texture override(s) needed when painting a mesh component
	 * This object will keep track of the overridden material resources and remove the override when they are no longer referred by an default texture override
	 */
	class FDefaultTextureOverride final
	{
	public:
		UE_API void ApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const;
		UE_DEPRECATED(5.7, "Use the version that takes a FMaterialUpdateContext.")
		UE_API void ApplyOrRemoveTextureOverride(UMeshComponent* InMeshComponent, UTexture* SourceTexture, UTexture* OverrideTexture) const;

		UE_API ~FDefaultTextureOverride();

		FDefaultTextureOverride() = default;

		UE_API FDefaultTextureOverride(const FDefaultTextureOverride& InOther);
		UE_API FDefaultTextureOverride(FDefaultTextureOverride&& InOther);

		UE_API FDefaultTextureOverride& operator=(const FDefaultTextureOverride& InOther);
		UE_API FDefaultTextureOverride& operator=(FDefaultTextureOverride&& InOther);
	};
}

#undef UE_API
