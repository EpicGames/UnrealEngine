// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API MESHPAINTINGTOOLSET_API

class UMeshComponent;
class IMeshPaintComponentAdapterFactory;
class FReferenceCollector;

class FMeshPaintComponentAdapterFactory
{
public:
	static UE_API TArray<TSharedPtr<IMeshPaintComponentAdapterFactory>> FactoryList;

public:
	static UE_API TSharedPtr<class IMeshPaintComponentAdapter> CreateAdapterForMesh(UMeshComponent* InComponent, int32 InPaintingMeshLODIndex);
	static UE_API void InitializeAdapterGlobals();
	static UE_API void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static UE_API void CleanupGlobals();
};

#undef UE_API
