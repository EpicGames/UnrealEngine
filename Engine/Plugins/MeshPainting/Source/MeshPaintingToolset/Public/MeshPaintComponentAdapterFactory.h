// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMeshPaintComponentAdapter;
class UMeshComponent;
class FReferenceCollector;

/**
 * Factory for IMeshPaintGeometryAdapter
 */
class IMeshPaintComponentAdapterFactory
{
public:
	virtual TSharedPtr<IMeshPaintComponentAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const = 0;
	virtual void InitializeAdapterGlobals() = 0;
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) = 0;
	virtual void CleanupGlobals() = 0;
	virtual ~IMeshPaintComponentAdapterFactory() = default;
};
