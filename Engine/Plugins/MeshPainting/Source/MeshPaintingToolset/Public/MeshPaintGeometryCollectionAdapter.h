// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMeshPaintComponentAdapter.h"
#include "MeshPaintComponentAdapterFactory.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API MESHPAINTINGTOOLSET_API

class UBodySetup;
class UGeometryCollection;
class UGeometryCollectionComponent;
class UTexture;
class UGeometryCollection;
class UMeshComponent;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshes
class FMeshPaintGeometryCollectionComponentAdapter : public FBaseMeshPaintComponentAdapter
{
public:
	static UE_API void InitializeAdapterGlobals();
	static UE_API void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static UE_API void CleanupGlobals();
	/** Start IMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	UE_API virtual ~FMeshPaintGeometryCollectionComponentAdapter();
	UE_API virtual bool Initialize() override;
	UE_API virtual void OnAdded() override;
	UE_API virtual void OnRemoved() override;
	UE_API virtual bool IsValid() const override;
	virtual bool SupportsTexturePaint() const override { return IsValid(); }
	virtual bool SupportsTextureColorPaint() const override { return false; }
	virtual bool SupportsVertexPaint() const override { return IsValid(); }
	UE_API virtual int32 GetNumUVChannels() const override;
	UE_API virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;
	UE_API virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	UE_API virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void PreEdit() override;
	UE_API virtual void PostEdit() override;
	UE_API virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	UE_API virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	UE_API virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	UE_API virtual FMatrix GetComponentToWorldMatrix() const override;
	/** End IMeshPaintGeometryAdapter Overrides */
		
	/** Start FBaseMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool InitializeVertexData();
	/** End FBaseMeshPaintGeometryAdapter Overrides */
protected:

	/** Delegate called when geometry collection is changed on the component */
	FDelegateHandle GeometryCollectionChangedHandle;

	/** Geometry Collection component represented by this adapter */
	TWeakObjectPtr<UGeometryCollectionComponent> GeometryCollectionComponent;

	bool bSavedShowBoneColors = false;

	// Texture override state
	UE::MeshPaintingToolset::FDefaultTextureOverride TextureOverridesState;

	/// Get the underlying UGeometryCollection from the component, as a non-const object
	/// Caller must have already validated that the component weak pointer is still valid (as this is called per-vertex)
	UE_API UGeometryCollection* GetGeometryCollectionObject() const;

	UE_API void OnGeometryCollectionChanged();


	// Like IsValid() but does not verify that the cached data matches
	UE_API bool HasValidGeometryCollection() const;
	
	// TODO: Store a LOD index if/when GeometryCollection supports LODs
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshesFactory

class FMeshPaintGeometryCollectionComponentAdapterFactory : public IMeshPaintComponentAdapterFactory
{
public:
	UE_API virtual TSharedPtr<IMeshPaintComponentAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintGeometryCollectionComponentAdapter::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintGeometryCollectionComponentAdapter::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintGeometryCollectionComponentAdapter::CleanupGlobals(); }
};

#undef UE_API
