// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/StaticMesh.h"
#include "ImageCore.h"
#include "Math/Ray.h"
#include "UObject/Package.h"
#include "MeshPaintHelpers.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

enum class EMeshPaintModeAction : uint8;

class FMeshPaintParameters;
class UImportVertexColorOptions;
class UTexture2D;
class UStaticMeshComponent;
class USkeletalMesh;
class IMeshPaintComponentAdapter;
class UPaintBrushSettings;
class FEditorViewportClient;
class UMeshComponent;
class USkeletalMeshComponent;
class UViewportInteractor;
class FViewport;
class FPrimitiveDrawInterface;
class FSceneView;
struct FStaticMeshComponentLODInfo;
class UMeshVertexPaintingToolProperties;
class UBrushBaseProperties;
struct FMeshDescription;
class UInteractiveTool;
class UInteractiveToolPropertySet;

enum class EMeshPaintDataColorViewMode : uint8;

/** struct used to store the color data copied from mesh instance to mesh instance */
struct FPerLODVertexColorData
{
	TArray< FColor > ColorsByIndex;
	TMap<FVector, FColor> ColorsByPosition;
};

/** struct used to store the color data copied from mesh component to mesh component */
struct FPerComponentVertexColorData
{
	FPerComponentVertexColorData(const UStaticMesh* InStaticMesh, int32 InComponentIndex)
		: OriginalMesh(InStaticMesh)
		, ComponentIndex(InComponentIndex)
	{
	}

	/** We match up components by the mesh they use */
	TWeakObjectPtr<const UStaticMesh> OriginalMesh;

	/** We also match by component index */
	int32 ComponentIndex;

	/** Vertex colors by LOD */
	TArray<FPerLODVertexColorData> PerLODVertexColorData;
};

/** Struct to hold MeshPaint settings on a per mesh basis */
struct FInstanceTexturePaintSettings
{
	UTexture2D* SelectedTexture;
	int32 SelectedUVChannel;

	FInstanceTexturePaintSettings()
		: SelectedTexture(nullptr)
		, SelectedUVChannel(0)
	{}
	FInstanceTexturePaintSettings(UTexture2D* InSelectedTexture, int32 InSelectedUVSet)
		: SelectedTexture(InSelectedTexture)
		, SelectedUVChannel(InSelectedUVSet)
	{}

	void operator=(const FInstanceTexturePaintSettings& SrcSettings)
	{
		SelectedTexture = SrcSettings.SelectedTexture;
		SelectedUVChannel = SrcSettings.SelectedUVChannel;
	}
};

/** Struct for some static helper functions to store tool properties. */
class FMeshPaintToolSettingHelpers
{
public:
	/** Store properties in a property set for later restore in a session. This saves settings per base class so that different class hierachies can share their base class data. */
	static void SavePropertiesForClassHeirachy(UInteractiveTool* InTool, UInteractiveToolPropertySet* InProperties);
	/** Restore properties in a property that were saved in this session. This restores settings per base class so that different class hierachies can share their base class data. */
	static void RestorePropertiesForClassHeirachy(UInteractiveTool* InTool, UInteractiveToolPropertySet* InProperties);
private:
	/** Get unique string used to identify property set in cache. */
	static TCHAR const* GetCacheIdentifier();
};

UENUM()
enum class ETexturePaintWeightTypes : uint8
{
	/** Lerp Between Two Textures using Alpha Value */
	AlphaLerp = 2 UMETA(DisplayName = "Alpha (Two Textures)"),

	/** Weighting Three Textures according to Channels*/
	RGB = 3 UMETA(DisplayName = "RGB (Three Textures)"),

	/**  Weighting Four Textures according to Channels*/
	ARGB = 4 UMETA(DisplayName = "ARGB (Four Textures)"),

	/**  Weighting Five Textures according to Channels */
	OneMinusARGB = 5 UMETA(DisplayName = "ARGB - 1 (Five Textures)")
};

UENUM()
enum class ETexturePaintWeightIndex : uint8
{
	TextureOne = 0,
	TextureTwo,
	TextureThree,
	TextureFour,
	TextureFive
};

/** Parameters for paint actions, stored together for convenience */
struct FPerVertexPaintActionArgs
{
	IMeshPaintComponentAdapter* Adapter;
	UMeshVertexPaintingToolProperties* BrushProperties;
	FVector CameraPosition;
	FHitResult HitResult;
	EMeshPaintModeAction Action;
};

/** Delegates used to call per-vertex/triangle actions */
DECLARE_DELEGATE_TwoParams(FPerVertexPaintAction, FPerVertexPaintActionArgs& /*Args*/, int32 /*VertexIndex*/);
DECLARE_DELEGATE_ThreeParams(FPerTrianglePaintAction, IMeshPaintComponentAdapter* /*Adapter*/, int32 /*TriangleIndex*/, const int32[3] /*Vertex Indices*/);

UCLASS(MinimalAPI)
class UMeshPaintingSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UE_API UMeshPaintingSubsystem();

	UE_API bool HasPaintableMesh(UActorComponent* Component);
	/** Removes vertex colors associated with the object */
	UE_API void RemoveInstanceVertexColors(UObject* Obj);

	/** Removes vertex colors associated with the mesh component */
	UE_API void RemoveComponentInstanceVertexColors(UStaticMeshComponent* StaticMeshComponent);

	/** Creates and returns a mesh paint texture that isn't attached to a mesh component */
	UE_API UTexture* CreateMeshPaintTexture(UObject* Outer, uint32 TextureSize);
	/** Creates mesh paint texture associated with the mesh component */
	UE_API void CreateComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent);
	UE_API void CreateComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent, FImageView const& InImage);

	/** Removes mesh paint texture associated with the mesh component */
	UE_API void RemoveComponentMeshPaintTexture(UStaticMeshComponent* StaticMeshComponent);

	/** Propagates per-instance vertex colors to the underlying Mesh for the given LOD Index */
	UE_API bool PropagateColorsToRawMesh(UStaticMesh* StaticMesh, int32 LODIndex, FStaticMeshComponentLODInfo& ComponentLODInfo);	

	/** Retrieves the Vertex Color buffer size for the given LOD level in the Mesh */
	UE_API uint32 GetVertexColorBufferSize(UMeshComponent* MeshComponent, int32 LODIndex, bool bInstance);
	
	/** Retrieves the resource size for the mesh paint texture on the component */
	UE_API uint32 GetMeshPaintTextureResourceSize(UMeshComponent* MeshComponent);

	/** Retrieves the vertex positions from the given LOD level in the Mesh */
	UE_API TArray<FVector> GetVerticesForLOD(const UStaticMesh* StaticMesh, int32 LODIndex);

	/** Retrieves the vertex colors from the given LOD level in the Mesh */
	UE_API TArray<FColor> GetColorDataForLOD(const UStaticMesh* StaticMesh, int32 LODIndex);

	/** Retrieves the per-instance vertex colors from the given LOD level in the StaticMeshComponent */
	UE_API TArray<FColor> GetInstanceColorDataForLOD(const UStaticMeshComponent* MeshComponent, int32 LODIndex);

	/** Sets the specific (LOD Index) per-instance vertex colors for the given StaticMeshComponent to the supplied Color array */
	UE_API void SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const TArray<FColor>& Colors);	
	
	/** Sets the specific (LOD Index) per-instance vertex colors for the given StaticMeshComponent to a single Color value */
	UE_API void SetInstanceColorDataForLOD(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	
	/** Fills all vertex colors for all LODs found in the given mesh component with Fill Color */
	UE_API void FillStaticMeshVertexColors(UStaticMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	UE_API void FillSkeletalMeshVertexColors(USkeletalMeshComponent* MeshComponent, int32 LODIndex, const FColor FillColor, const FColor MaskColor);
	
	/** Sets all vertex colors for a specific LOD level in the SkeletalMesh to FillColor */
	UE_API void SetColorDataForLOD(USkeletalMesh* SkeletalMesh, int32 LODIndex, const FColor FillColor, const FColor MaskColor);

	UE_API void ApplyFillWithMask(FColor& InOutColor, const FColor& MaskColor, const FColor& FillColor);

	/** Forces the component to render LOD level at LODIndex instead of the view-based LOD level ( X = 0 means do not force the LOD, X > 0 means force the lod to X - 1 ) */
	UE_API void ForceRenderMeshLOD(UMeshComponent* Component, int32 LODIndex);

	UE_DEPRECATED(5.7, "No longer supported. Will be a no-op.")
	UE_API void ClearMeshTextureOverrides(const IMeshPaintComponentAdapter& GeometryInfo, UMeshComponent* InMeshComponent);

	/** Applies vertex color painting found on LOD 0 to all lower LODs. */
	UE_API void ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, UMeshComponent* InMeshComponent);

	/** Applies the vertex colors found in LOD level 0 to all contained LOD levels in the StaticMeshComponent */
	UE_API void ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, UStaticMeshComponent* StaticMeshComponent);

	/** Applies the vertex colors found in LOD level 0 to all contained LOD levels in the SkeletalMeshComponent */
	UE_API void ApplyVertexColorsToAllLODs(IMeshPaintComponentAdapter& GeometryInfo, USkeletalMeshComponent* SkeletalMeshComponent);

	/** Returns the number of Mesh LODs for the given MeshComponent */
	UE_API int32 GetNumberOfLODs(const UMeshComponent* MeshComponent);

	/** OutNumLODs is set to number of Mesh LODs for the given MeshComponent and returns true, or returns false of given mesh component has no valid LODs */
	UE_API bool TryGetNumberOfLODs(const UMeshComponent* MeshComponent, int32& OutNumLODs);
	
	/** Returns the number of Texture Coordinates for the given MeshComponent */
	UE_API int32 GetNumberOfUVs(const UMeshComponent* MeshComponent, int32 LODIndex) const;

	/** Checks whether or not the mesh components contains per lod colors (for all LODs)*/
	UE_API bool DoesMeshComponentContainPerLODColors(const UMeshComponent* MeshComponent);

	/** Retrieves the number of bytes used to store the per-instance LOD vertex color data from the mesh component */
	UE_API void GetInstanceColorDataInfo(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, int32& OutTotalInstanceVertexColorBytes);

	/** Given arguments for an action, and an action - retrieves influences vertices and applies Action to them */
	UE_API bool ApplyPerVertexPaintAction(FPerVertexPaintActionArgs& InArgs, FPerVertexPaintAction Action);

	UE_API bool GetPerVertexPaintInfluencedVertices(FPerVertexPaintActionArgs& InArgs, TSet<int32>& InfluencedVertices);

	/** Given the adapter, settings and view-information retrieves influences triangles and applies Action to them */
	UE_API bool ApplyPerTrianglePaintAction(IMeshPaintComponentAdapter* Adapter, const FVector& CameraPosition, const FVector& HitPosition, const UBrushBaseProperties* Settings, FPerTrianglePaintAction Action, bool bOnlyFrontFacingTriangles);

	/** Applies vertex painting to InOutvertexColor according to the given parameters  */
	UE_API bool PaintVertex(const FVector& InVertexPosition, const FMeshPaintParameters& InParams, FColor& InOutVertexColor);

	/** Applies Vertex Color Painting according to the given parameters */
	UE_API void ApplyVertexColorPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount);

	/** Applies Vertex Blend Weight Painting according to the given parameters */
	UE_API void ApplyVertexWeightPaint(const FMeshPaintParameters &InParams, const FLinearColor &OldColor, FLinearColor &NewColor, const float PaintAmount);

	/** Generate texture weight color for given number of weights and the to-paint index */
	UE_API FLinearColor GenerateColorForTextureWeight(const int32 NumWeights, const int32 WeightIndex);

	/** Computes the Paint power multiplier value */
	UE_API float ComputePaintMultiplier(float SquaredDistanceToVertex2D, float BrushStrength, float BrushInnerRadius, float BrushRadialFalloff, float BrushInnerDepth, float BrushDepthFallof, float VertexDepthToBrush);

	/** Checks whether or not a point is influenced by the painting brush according to the given parameters*/
	UE_API bool IsPointInfluencedByBrush(const FVector& InPosition, const FMeshPaintParameters& InParams, float& OutSquaredDistanceToVertex2D, float& OutVertexDepthToBrush);

	UE_API bool IsPointInfluencedByBrush(const FVector2D& BrushSpacePosition, const float BrushRadiusSquared, float& OutInRangeValue);

	template<typename T>
	void ApplyBrushToVertex(const FVector& VertexPosition, const FMatrix& InverseBrushMatrix, const float BrushRadius, const float BrushFalloffAmount, const float BrushStrength, const T& PaintValue, T& InOutValue);

	/** Helper function to retrieve vertex color from a UTexture given a UVCoordinate */
	UE_API FColor PickVertexColorFromTextureData(const uint8* MipData, const FVector2D& UVCoordinate, const UTexture2D* Texture, const FColor ColorMask);	

	/** Map of geometry adapters for each selected mesh component */
	UE_API TSharedPtr<IMeshPaintComponentAdapter> GetAdapterForComponent(const UMeshComponent* InComponent) const;
	UE_API void AddToComponentToAdapterMap(const UMeshComponent* InComponent, const TSharedPtr<IMeshPaintComponentAdapter> InAdapter);

	UE_API TArray<UMeshComponent*> GetSelectedMeshComponents() const;
	UE_API void AddSelectedMeshComponents(const TArray<UMeshComponent*>& InComponents);
	UE_API bool FindHitResult(const FRay Ray, FHitResult& BestTraceResult);
	UE_API void ClearSelectedMeshComponents();
	UE_API TArray<UMeshComponent*> GetPaintableMeshComponents() const;
	UE_API void AddPaintableMeshComponent(UMeshComponent* InComponent);
	UE_API void ClearPaintableMeshComponents();
	UE_API TArray<FPerComponentVertexColorData> GetCopiedColorsByComponent() const;
	UE_API void SetCopiedColorsByComponent(TArray<FPerComponentVertexColorData>& InCopiedColors);
	UE_API void CacheSelectionData(const int32 PaintLODIndex, const int32 UVChannel);
	UE_API FIntPoint GetMinMaxUVChannelsToPaint() const;
	UE_API void ResetState();
	UE_API void Refresh();
	bool SelectionContainsPerLODColors() const { return bSelectionContainsPerLODColors; }
	void ClearSelectionLODColors() { bSelectionContainsPerLODColors = false; }
	UE_API void UpdatePaintSupportState();
	bool GetSelectionSupportsVertexPaint() const { return bSelectionSupportsVertexPaint; }
	bool GetSelectionSupportsTextureColorPaint() const { return bSelectionSupportsTextureColorPaint; }
	bool GetSelectionSupportsTextureAssetPaint() const { return bSelectionSupportsTextureAssetPaint; }

	UE_API FImage const& GetCopiedTexture() const;
	UE_API void SetCopiedTexture(UTexture* InTexture);

public:
	bool bNeedsRecache;

	UPROPERTY(Transient)
	TObjectPtr<UMeshComponent> LastPaintedComponent;

private:
	UE_API void CleanUp();

private:
	/** Map of geometry adapters for each selected mesh component */
	TMap<FString, TSharedPtr<IMeshPaintComponentAdapter>> ComponentToAdapterMap;

	/** Currently selected mesh components as provided by the mode class */
	TArray<TWeakObjectPtr<UMeshComponent>> SelectedMeshComponents;

	/** Mesh components within the current selection which are eligible for painting */
	TArray<TWeakObjectPtr<UMeshComponent>> PaintableComponents;

	/** Contains copied vertex color data */
	TArray<FPerComponentVertexColorData> CopiedColorsByComponent;
	bool bSelectionContainsPerLODColors;

	/** Contains copied texture data */
	FImage CopiedTextureData;

	bool bSelectionSupportsVertexPaint;
	bool bSelectionSupportsTextureColorPaint;
	bool bSelectionSupportsTextureAssetPaint;
};

template<typename T>
void UMeshPaintingSubsystem::ApplyBrushToVertex(const FVector& VertexPosition, const FMatrix& InverseBrushMatrix, const float BrushRadius, const float BrushFalloffAmount, const float BrushStrength, const T& PaintValue, T& InOutValue)
{
	const FVector BrushSpacePosition = InverseBrushMatrix.TransformPosition(VertexPosition);
	const FVector2D BrushSpacePosition2D(BrushSpacePosition.X, BrushSpacePosition.Y);
		
	float InfluencedValue = 0.0f;
	if (IsPointInfluencedByBrush(BrushSpacePosition2D, BrushRadius * BrushRadius, InfluencedValue))
	{
		float InnerBrushRadius = BrushFalloffAmount * BrushRadius;
		float PaintStrength = GEngine->GetEngineSubsystem<UMeshPaintingSubsystem>()->ComputePaintMultiplier(BrushSpacePosition2D.SizeSquared(), BrushStrength, InnerBrushRadius, BrushRadius - InnerBrushRadius, 1.0f, 1.0f, 1.0f);

		const T OldValue = InOutValue;
		InOutValue = FMath::LerpStable(OldValue, PaintValue, PaintStrength);
	}	
};

#undef UE_API
