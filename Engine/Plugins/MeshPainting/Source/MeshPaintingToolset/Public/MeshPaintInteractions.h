// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractionMechanic.h"
#include "UObject/Interface.h"
#include "MeshPaintInteractions.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

struct FInputDeviceRay;
struct FInputRayHit;

class IMeshPaintComponentAdapter;
class AActor;
class UMeshComponent;
class UMeshToolManager;

UINTERFACE(MinimalAPI)
class UMeshPaintSelectionInterface : public UInterface
{
	GENERATED_BODY()
};

class IMeshPaintSelectionInterface
{
	GENERATED_BODY()
public:
	virtual bool IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const PURE_VIRTUAL(IMeshPaintSelectionInterface::IsMeshAdapterSupported, return false;);
	virtual bool AllowsMultiselect() const PURE_VIRTUAL(IMeshPaintSelectionInterface::AllowsMultiselect, return false;);
};


UCLASS(MinimalAPI)
class UMeshPaintSelectionMechanic : public UInteractionMechanic
{
	GENERATED_BODY()

public:
	UE_API FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos, bool bIsFallbackClick = false);
	UE_API void OnClicked(const FInputDeviceRay& ClickPos);
	void SetAddToSelectionSet(const bool bInNewSelectionType)
	{
		bAddToSelectionSet = bInNewSelectionType;
	}
protected:
	UE_API bool FindClickedComponentsAndCacheAdapters(const FInputDeviceRay& ClickPos);

protected:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMeshComponent>> CachedClickedComponents;
	UPROPERTY(Transient)
	TArray<TObjectPtr<AActor>> CachedClickedActors;
	bool bAddToSelectionSet;
};

#undef UE_API
