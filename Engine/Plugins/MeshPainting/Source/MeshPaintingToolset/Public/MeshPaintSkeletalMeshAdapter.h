// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "BaseMeshPaintComponentAdapter.h"
#include "MeshPaintComponentAdapterFactory.h"

#define UE_API MESHPAINTINGTOOLSET_API

class UBodySetup;
class USkeletalMesh;
class USkeletalMeshComponent;
class UTexture;
class FSkeletalMeshRenderData;
class FSkeletalMeshLODRenderData;
class FSkeletalMeshLODModel;
class UMeshComponent;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshes
class FMeshPaintSkeletalMeshComponentAdapter : public FBaseMeshPaintComponentAdapter
{
public:
	static UE_API void InitializeAdapterGlobals();
	static UE_API void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static UE_API void CleanupGlobals();
	/** Start IMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	UE_API virtual ~FMeshPaintSkeletalMeshComponentAdapter();
	UE_API virtual bool Initialize() override;
	UE_API virtual void OnAdded() override;
	UE_API virtual void OnRemoved() override;
	virtual bool IsValid() const override { return SkeletalMeshComponent.IsValid() && ReferencedSkeletalMesh && SkeletalMeshComponent->GetSkeletalMeshAsset() == ReferencedSkeletalMesh; }
	virtual bool SupportsTexturePaint() const override { return true; }
	virtual bool SupportsTextureColorPaint() const override { return false; }
	virtual bool SupportsVertexPaint() const override { return SkeletalMeshComponent.IsValid(); }
	UE_API virtual int32 GetNumUVChannels() const override;
	UE_API virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;
	UE_API virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	UE_API virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void PreEdit() override;
	UE_API virtual void PostEdit() override;
	UE_API virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	UE_API virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	UE_API virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	UE_API virtual FMatrix GetComponentToWorldMatrix() const override;
	/** End IMeshPaintGeometryAdapter Overrides */
		
	/** Start FBaseMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool InitializeVertexData();
	/** End FBaseMeshPaintGeometryAdapter Overrides */
protected:
	/** Callback for when the skeletal mesh on the component is changed */
	UE_API void OnSkeletalMeshChanged();
	/** Callback for when skeletal mesh DDC data is rebuild */
	UE_API void OnPostMeshCached(USkeletalMesh* SkeletalMesh);

	/** Delegate called when skeletal mesh is changed on the component */
	FDelegateHandle SkeletalMeshChangedHandle;	

	/** Skeletal mesh component represented by this adapter */
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
	/** Skeletal mesh currently set to the Skeletal Mesh Component */
	TObjectPtr<USkeletalMesh> ReferencedSkeletalMesh;
	/** Skeletal Mesh resource retrieved from the Skeletal Mesh */
	FSkeletalMeshRenderData* MeshResource;

	/** LOD render data (at Mesh LOD Index) containing data to change */
	FSkeletalMeshLODRenderData* LODData;
	/** LOD model (source) data (at Mesh LOD Index) containing data to change */
	FSkeletalMeshLODModel* LODModel;
	/** LOD Index for which data has to be retrieved / altered*/
	int32 MeshLODIndex;

	// Texture override state
	UE::MeshPaintingToolset::FDefaultTextureOverride TextureOverridesState;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSkeletalMeshesFactory

class FMeshPaintSkeletalMeshComponentAdapterFactory : public IMeshPaintComponentAdapterFactory
{
public:
	UE_API virtual TSharedPtr<IMeshPaintComponentAdapter> Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintSkeletalMeshComponentAdapter::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintSkeletalMeshComponentAdapter::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintSkeletalMeshComponentAdapter::CleanupGlobals(); }
};

#undef UE_API
