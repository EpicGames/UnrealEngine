// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPaintStaticMeshAdapter.h"

#define UE_API MESHPAINTINGTOOLSET_API

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshes

class FMeshPaintSplineMeshComponentAdapter : public FMeshPaintStaticMeshComponentAdapter
{
public:
	UE_API virtual bool InitializeVertexData() override;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForSplineMeshesFactory

class FMeshPaintSplineMeshComponentAdapterFactory : public FMeshPaintStaticMeshComponentAdapterFactory
{
public:
	UE_API virtual TSharedPtr<IMeshPaintComponentAdapter> Construct(class UMeshComponent* InComponent, int32 MeshLODIndex) const override;
};

#undef UE_API
