// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMeshPaintComponentAdapter.h"
#include "Components/StaticMeshComponent.h"
#include "MeshPaintComponentAdapterFactory.h"

#define UE_API MESHPAINTINGTOOLSET_API

class UBodySetup;
class UStaticMesh;
class UTexture;
struct FStaticMeshLODResources;

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshes

class FMeshPaintStaticMeshComponentAdapter : public FBaseMeshPaintComponentAdapter
{
public:
	static UE_API void InitializeAdapterGlobals();
	static UE_API void AddReferencedObjectsGlobals(FReferenceCollector& Collector);
	static UE_API void CleanupGlobals();
	/** Start IMeshPaintGeometryAdapter Overrides */
	UE_API virtual bool Construct(UMeshComponent* InComponent, int32 InMeshLODIndex) override;
	UE_API virtual ~FMeshPaintStaticMeshComponentAdapter();
	UE_API virtual bool Initialize() override;
	virtual void OnAdded() override {}
	virtual void OnRemoved() override {}
	virtual bool IsValid() const override { return StaticMeshComponent.IsValid() && ReferencedStaticMesh && StaticMeshComponent->GetStaticMesh() == ReferencedStaticMesh; }
	virtual bool SupportsTexturePaint() const override { return true; }
	virtual bool SupportsTextureColorPaint() const override { return StaticMeshComponent.IsValid() && StaticMeshComponent->CanMeshPaintTextureColors(); }
	UE_API virtual int32 GetNumUVChannels() const override;
	virtual bool SupportsVertexPaint() const override { return StaticMeshComponent.IsValid() && StaticMeshComponent->CanMeshPaintVertexColors(); }
	UE_API virtual bool LineTraceComponent(struct FHitResult& OutHit, const FVector Start, const FVector End, const struct FCollisionQueryParams& Params) const override;	
	UE_API virtual void QueryPaintableTextures(int32 MaterialIndex, int32& OutDefaultIndex, TArray<struct FPaintableTexture>& InOutTextureList) override;
	UE_API virtual void ApplyOrRemoveTextureOverride(UTexture* SourceTexture, UTexture* OverrideTexture, FMaterialUpdateContext& MaterialUpdateContext) const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual void GetTextureCoordinate(int32 VertexIndex, int32 ChannelIndex, FVector2D& OutTextureCoordinate) const override;
	UE_API virtual void GetVertexColor(int32 VertexIndex, FColor& OutColor, bool bInstance = true) const override;
	UE_API virtual void SetVertexColor(int32 VertexIndex, FColor Color, bool bInstance = true) override;
	UE_API virtual FMatrix GetComponentToWorldMatrix() const override;
	UE_API virtual void PreEdit() override;
	UE_API virtual void PostEdit() override;
	/** End IMeshPaintGeometryAdapter Overrides*/

	/** Begin FMeshBasePaintGeometryAdapter */
	UE_API virtual bool InitializeVertexData() override;	
	/** End FMeshBasePaintGeometryAdapter */

protected:
	/** Callback for when the static mesh data is rebuilt */
	UE_API void OnPostMeshBuild(UStaticMesh* StaticMesh);
	/** Callback for when the static mesh on the component is changed */
	UE_API void OnStaticMeshChanged(UStaticMeshComponent* StaticMeshComponent);

	/** Static mesh component represented by this adapter */
	TWeakObjectPtr<UStaticMeshComponent> StaticMeshComponent;
	/** Static mesh currently set to the Static Mesh Component */
	TObjectPtr<UStaticMesh> ReferencedStaticMesh;
	/** LOD model (at Mesh LOD Index) containing data to change */
	FStaticMeshLODResources* LODModel;
	/** LOD Index for which data has to be retrieved / altered*/
	int32 MeshLODIndex;

	// Texture override state
	UE::MeshPaintingToolset::FDefaultTextureOverride TextureOverridesState;
};

//////////////////////////////////////////////////////////////////////////
// FMeshPaintGeometryAdapterForStaticMeshesFactory

class FMeshPaintStaticMeshComponentAdapterFactory : public IMeshPaintComponentAdapterFactory
{
public:
	UE_API virtual TSharedPtr<IMeshPaintComponentAdapter> Construct(class UMeshComponent* InComponent, int32 InMeshLODIndex) const override;
	virtual void InitializeAdapterGlobals() override { FMeshPaintStaticMeshComponentAdapter::InitializeAdapterGlobals(); }
	virtual void AddReferencedObjectsGlobals(FReferenceCollector& Collector) override { FMeshPaintStaticMeshComponentAdapter::AddReferencedObjectsGlobals(Collector); }
	virtual void CleanupGlobals() override { FMeshPaintStaticMeshComponentAdapter::CleanupGlobals(); }
};

#undef UE_API
