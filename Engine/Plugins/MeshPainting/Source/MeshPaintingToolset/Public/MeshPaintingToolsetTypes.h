// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Requires.h"

#include <type_traits>

#include "MeshPaintingToolsetTypes.generated.h"

class UMeshComponent;
class UTexture2D;
class UTexture;
class UTextureRenderTarget2D;

class FMeshPaintParameters;
DECLARE_MULTICAST_DELEGATE_FourParams(FApplyVertexPaintData, const FMeshPaintParameters& /* &InParams*/, const FLinearColor& /*OldColor*/, FLinearColor& /*NewColor*/, const float /*PaintAmount*/);

/** Mesh paint color view modes (somewhat maps to EVertexColorViewMode engine enum.) */
UENUM()
enum class EMeshPaintDataColorViewMode : uint8
{
	/** Normal view mode (vertex color visualization off) */
	Normal UMETA(DisplayName = "Off"),

	/** RGB only */
	RGB UMETA(DisplayName = "RGB Channels"),

	/** Alpha only */
	Alpha UMETA(DisplayName = "Alpha Channel"),

	/** Red only */
	Red UMETA(DisplayName = "Red Channel"),

	/** Green only */
	Green UMETA(DisplayName = "Green Channel"),

	/** Blue only */
	Blue UMETA(DisplayName = "Blue Channel"),
};

/** Mesh painting action (paint, erase) */
enum class EMeshPaintModeAction : uint8
{
	/** Paint (add color or increase blending weight) */
	Paint,

	/** Erase (remove color or decrease blending weight) */
	Erase
};

/** Mesh paint parameters */
class FMeshPaintParameters
{
public:
	EMeshPaintModeAction PaintAction;
	FVector BrushPosition;
	FVector BrushNormal;
	FLinearColor BrushColor;
	float SquaredBrushRadius;
	float BrushRadialFalloffRange;
	float InnerBrushRadius;
	float BrushDepth;
	float BrushDepthFalloffRange;
	float InnerBrushDepth;
	float BrushStrength;
	FMatrix BrushToWorldMatrix;
	FMatrix InverseBrushToWorldMatrix;
	bool bWriteRed;
	bool bWriteGreen;
	bool bWriteBlue;
	bool bWriteAlpha;
	int32 TotalWeightCount;
	int32 PaintWeightIndex;
	FApplyVertexPaintData ApplyVertexDataDelegate;
	FVector2f BrushPosition2D;
	bool bUseFillBucket = false;
};

/** Structure used to hold per-triangle data for texture painting */
struct FTexturePaintTriangleInfo
{
	FVector TriVertices[3];
	FVector2D TrianglePoints[3];
	FVector2D TriUVs[3];
};

/** Structure used to house and compare Texture and UV channel pairs */
struct FPaintableTexture
{
	UTexture*	Texture;
	int32		UVChannelIndex;
	bool		bIsMeshTexture;

	FPaintableTexture()
		: Texture(nullptr)
		, UVChannelIndex(0)
		, bIsMeshTexture(false)
	{}

	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<T, UTexture*>)
	>
	FPaintableTexture(T InTexture = nullptr, uint32 InUVChannelIndex = 0, bool bInIsMeshTexture = false)
		: Texture(InTexture)
		, UVChannelIndex(InUVChannelIndex)
		, bIsMeshTexture(bInIsMeshTexture)
	{}

	/** Overloaded equality operator for use with TArrays Contains method. */
	bool operator==(const FPaintableTexture& rhs) const
	{
		return Texture == rhs.Texture && UVChannelIndex == rhs.UVChannelIndex && bIsMeshTexture == rhs.bIsMeshTexture;
	}
};

USTRUCT()
struct FPaintTexture2DData
{
	GENERATED_USTRUCT_BODY()

	FPaintTexture2DData() = default;

	FPaintTexture2DData(UTexture2D* InPaintingTexture2D)
		: PaintingTexture2D(InPaintingTexture2D)
	{
	}

	/** The original texture that we're painting */
	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> PaintingTexture2D;

	/** Render target texture for painting */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PaintRenderTargetTexture;

	/** Optional virtual texture adapter that we can use to visualize PaintRenderTargetTexture in materials that sample virtual textures */
	UPROPERTY(Transient)
	TObjectPtr<class UVirtualTextureAdapter> PaintRenderTargetTextureAdapter;

	/** Optional render target texture used as an input while painting that contains a clone of the texture painting brush */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PaintBrushRenderTargetTexture;

	/** Temporary render target used to draw incremental paint to */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> BrushRenderTargetTexture;

	/** Temporary render target used to store a mask of the affected paint region, updated every time we add incremental texture paint */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> BrushMaskRenderTargetTexture;

	/** Optional render target used to store generated mask for texture seams. We create this by projecting object triangles into texture space using the selected UV channel. */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> SeamMaskRenderTargetTexture;

	/** Array of components that have the PaintRenderTargetTexture set as a texture override */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMeshComponent>> TextureOverrideComponents;

	/** Array of components that have been painted */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMeshComponent>> PaintedComponents;

	/** Current component that was used to generate the texture seam mask used for texture dilation */
	TObjectPtr<UMeshComponent> SeamMaskComponent;

	/** True if we have some painting applied to the PaintRenderTargetTexture. */
	bool bIsPaintingTexture2DModified = false;
};

namespace MeshPaintDefs
{
	// Design constraints

	// Currently we never support more than five channels (R, G, B, A, OneMinusTotal)
	static const int32 MaxSupportedPhysicalWeights = 4;
	static const int32 MaxSupportedWeights = MaxSupportedPhysicalWeights + 1;
}
