// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleClickTool.h"
#include "MeshPaintInteractions.h"
#include "MeshSelect.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

class IMeshPaintComponentAdapter;
class AActor;
class UMeshComponent;
class UMeshPaintSelectionMechanic;

/**
 * Builder for UVertexAdapterClickTool
 */
UCLASS(MinimalAPI)
class UVertexAdapterClickToolBuilder : public USingleClickToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Builder for UTextureColorAdapterClickTool
 */
UCLASS(MinimalAPI)
class UTextureColorAdapterClickToolBuilder : public USingleClickToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Builder for UTextureAssetAdapterClickTool
 */
UCLASS(MinimalAPI)
class UTextureAssetAdapterClickToolBuilder : public USingleClickToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * USingleClickTool is perhaps the simplest possible interactive tool. It simply
 * reacts to default primary button clicks for the active device (eg left-mouse clicks).
 *
 * The function ::IsHitByClick() determines what is clickable by this Tool. The default is
 * to return true, which means the click will activate anywhere (the Tool itself has no
 * notion of Actors, Components, etc). You can override this function to, for example,
 * filter out clicks that don't hit a target object, etc.
 *
 * The function ::OnClicked() implements the action that will occur when a click happens.
 * You must override this to implement any kind of useful behavior.
 */
UCLASS(MinimalAPI, Abstract)
class UMeshClickTool : public USingleClickTool, public IMeshPaintSelectionInterface
{
	GENERATED_BODY()

public:
	UE_API UMeshClickTool();

	// USingleClickTool overrides
	UE_API virtual	void Setup() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	UE_API virtual bool IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const override;
	virtual bool AllowsMultiselect() const override { return false; }

protected:
	// flags used to identify modifier keys/buttons
	static const int AdditiveSelectionModifier = 1;


	UPROPERTY(Transient)
	TObjectPtr<UMeshPaintSelectionMechanic> SelectionMechanic;
};

UCLASS(MinimalAPI)
class UVertexAdapterClickTool : public UMeshClickTool
{
	GENERATED_BODY()

public:
	virtual bool AllowsMultiselect() const override { return true; }
};

UCLASS(MinimalAPI)
class UTextureColorAdapterClickTool : public UMeshClickTool
{
	GENERATED_BODY()

public:
	virtual bool AllowsMultiselect() const override { return true; }
};

UCLASS(MinimalAPI)
class UTextureAssetAdapterClickTool : public UMeshClickTool
{
	GENERATED_BODY()
};

#undef UE_API
