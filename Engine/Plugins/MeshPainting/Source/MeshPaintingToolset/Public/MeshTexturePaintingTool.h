// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/BaseBrushTool.h"
#include "BaseMeshPaintingToolProperties.h"
#include "MeshPaintingToolsetTypes.h"
#include "MeshPaintInteractions.h"
#include "MeshTexturePaintingTool.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

enum class EMeshPaintModeAction : uint8;
enum class EToolShutdownType : uint8;
class FMaterialUpdateContext;
class FScopedTransaction;
class IMeshPaintComponentAdapter;
class UMeshPaintChannelSet;
class UMeshToolManager;
class UTexture2D;
struct FPaintRayResults;
struct FPaintTexture2DData;
struct FTexturePaintMeshSectionInfo;
struct FToolBuilderState;


/**
 * Builder for the texture color mesh paint tool.
 */
UCLASS(MinimalAPI)
class UMeshTextureColorPaintingToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	TWeakObjectPtr<UMeshToolManager> SharedMeshToolData;
};

/**
 * Builder for the texture asset mesh paint tool.
 */
UCLASS(MinimalAPI)
class UMeshTextureAssetPaintingToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	TWeakObjectPtr<UMeshToolManager> SharedMeshToolData;
};


/**
 * Base class for mesh texture paint properties.
 */
UCLASS(MinimalAPI)
class UMeshTexturePaintingToolProperties : public UMeshPaintingToolProperties
{
	GENERATED_BODY()

public:
	/** Seam painting flag, True if we should enable dilation to allow the painting of texture seams */
	UPROPERTY(EditAnywhere, Category = TexturePainting)
	bool bEnableSeamPainting = false;

	/** Optional Texture Brush to which Painting should use */
	UPROPERTY(EditAnywhere, Category = TexturePainting, meta = (DisplayThumbnail = "true", TransientToolProperty))
	TObjectPtr<UTexture2D> PaintBrush = nullptr;

	/** Initial Rotation offset to apply to our paint brush */
	UPROPERTY(EditAnywhere, Category = TexturePainting, meta = (TransientToolProperty, UIMin = "0.0", UIMax = "360.0", ClampMin = "0.0", ClampMax = "360.0"))
	float PaintBrushRotationOffset = 0.0f;

	/** Whether or not to continously rotate the brush towards the painting direction */
	UPROPERTY(EditAnywhere, Category = TexturePainting, meta = (TransientToolProperty))
	bool bRotateBrushTowardsDirection = false;

	/** Whether or not to apply Texture Color Painting to the Red Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, meta = (DisplayName = "Red"))
	bool bWriteRed = true;

	/** Whether or not to apply Texture Color Painting to the Green Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, meta = (DisplayName = "Green"))
	bool bWriteGreen = true;

	/** Whether or not to apply Texture Color Painting to the Blue Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, meta = (DisplayName = "Blue"))
	bool bWriteBlue = true;

	/** Whether or not to apply Texture Color Painting to the Alpha Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, meta = (DisplayName = "Alpha"))
	bool bWriteAlpha = false;

	/** Get all of the write mask properties. */
	virtual void GetWriteMask(bool& bOutWriteRed, bool& bOutWriteGreen, bool& bOutWriteBlue, bool& bOutWriteAlpha) const
	{
		bOutWriteRed = bWriteRed;
		bOutWriteGreen = bWriteGreen;
		bOutWriteBlue = bWriteBlue;
		bOutWriteAlpha = bWriteAlpha;
	}
};

/**
 * Class for texture color paint properties.
 */
UCLASS(MinimalAPI)
class UMeshTextureColorPaintingToolProperties : public UMeshTexturePaintingToolProperties
{
	GENERATED_BODY()

public:
	/** Whether to copy all texture color painting to vertex colors. */
	UPROPERTY(EditAnywhere, Category = ColorPainting)
	bool bPropagateToVertexColor = false;
};

/**
 * Class for texture asset paint properties.
 */
UCLASS(MinimalAPI)
class UMeshTextureAssetPaintingToolProperties : public UMeshTexturePaintingToolProperties
{
	GENERATED_BODY()

public:
	/** The MeshPaintChannelSet asset that defines how a group of chanels map onto texture assets. */
	UPROPERTY(EditAnywhere, Category = WorldChannels, meta = (DisplayThumbnail = "false"))
	TObjectPtr<UMeshPaintChannelSet> ChannelSet;

	/** The currently selected Channel from the ChannelSet. */
	UPROPERTY(EditAnywhere, Category = WorldChannels, meta = (GetOptions = "GetChannelNames", editcondition = "ChannelSet != nullptr"))
	FName ChannelName;

	/** UV channel which should be used for painting textures. */
	UPROPERTY(EditAnywhere, Category = TexturePainting, meta = (TransientToolProperty, editcondition = "ChannelSet == nullptr"))
	int32 UVChannel = 0;

	/** Texture to which painting should be applied. */
	UPROPERTY(EditAnywhere, Category = TexturePainting, meta = (TransientToolProperty, DisplayThumbnail = "true", editcondition = "ChannelSet == nullptr"))
	TObjectPtr<UTexture2D> PaintTexture;

protected:
	/** Helper function to get the ChannelNames from the currently selected ChannelSet. */
	UFUNCTION()
    TArray<FName> GetChannelNames() const;

#if WITH_EDITOR
	// Begin UObject Interface.
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	// End UObject Interface.
#endif

	// Begin UMeshTexturePaintingToolProperties Interface.
	virtual void GetWriteMask(bool& bOutWriteRed, bool& bOutWriteGreen, bool& bOutWriteBlue, bool& bOutWriteAlpha) const override;
	// End UMeshTexturePaintingToolProperties Interface.
};


/**
 * Base class for mesh texture painting tool.
 */
UCLASS(MinimalAPI, Abstract)
class UMeshTexturePaintingTool : public UBaseBrushTool, public IMeshPaintSelectionInterface
{
	GENERATED_BODY()

public:
	UE_API UMeshTexturePaintingTool();

	DECLARE_DELEGATE_OneParam(FOnPaintingFinishedDelegate, UMeshComponent*);
	FOnPaintingFinishedDelegate& OnPaintingFinished() { return OnPaintingFinishedDelegate; }

	UE_API void FloodCurrentPaintTexture();

	virtual void GetModifiedTexturesToSave(TArray<UObject*>& OutTexturesToSave) const {}
	virtual int32 GetSelectedUVChannel(UMeshComponent const* InMeshComponent) const { return 0; }

protected:
	// Begin UInteractiveTool Interface.
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual	bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	UE_API virtual double EstimateMaximumTargetDimension() override;
	// End UInteractiveTool Interface.

	UE_API FPaintTexture2DData* GetPaintTargetData(const UTexture2D* InTexture);
	UE_API FPaintTexture2DData* AddPaintTargetData(UTexture2D* InTexture);
	
	UE_API void SetAllTextureOverrides(FMaterialUpdateContext& MaterialUpdateContext);
	UE_API void ClearAllTextureOverrides(FMaterialUpdateContext& MaterialUpdateContext);

	virtual UTexture2D* GetSelectedPaintTexture(UMeshComponent const* InMeshComponent) const { return nullptr; }
	virtual void CacheTexturePaintData() {}
	virtual bool CanPaintTextureToComponent(UTexture* InTexture, UMeshComponent const* InMeshComponent) const { return false; }
	virtual void ApplyWorldChannelUVs(TArrayView<FTexturePaintTriangleInfo> TriangleInfo) const {}

private:
	void CacheSelectionData();
	void AddTextureOverrideToComponent(FPaintTexture2DData& TextureData, UMeshComponent* MeshComponent, const IMeshPaintComponentAdapter* MeshPaintAdapter, FMaterialUpdateContext& MaterialUpdateContext);
	double CalculateTargetEdgeLength(int TargetTriCount);
	void StartPaintingTexture(UMeshComponent* InMeshComponent, const IMeshPaintComponentAdapter& GeometryInfo);
	void UpdateResult();
	void GatherTextureTriangles(IMeshPaintComponentAdapter* Adapter, int32 TriangleIndex, const int32 VertexIndices[3], TArray<FTexturePaintTriangleInfo>* TriangleInfo, TArray<FTexturePaintMeshSectionInfo>* SectionInfos, int32 UVChannelIndex);
	bool Paint(const FVector& InRayOrigin, const FVector& InRayDirection);
	bool Paint(const TArrayView<TPair<FVector, FVector>>& Rays);
	void PaintTexture(FMeshPaintParameters& InParams, int32 UVChannel, TArray<FTexturePaintTriangleInfo>& InInfluencedTriangles, UMeshComponent* MeshComponent, const IMeshPaintComponentAdapter& GeometryInfo, FMeshPaintParameters* LastParams = nullptr);
	bool PaintInternal(const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintModeAction PaintAction, float PaintStrength);
	void FinishPaintingTexture();
	void FinishPainting();

protected:
	/** Textures eligible for painting retrieved from the current selection */
	TArray<FPaintableTexture> PaintableTextures;

	/** Stores data associated with our paint target textures */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UTexture2D>, FPaintTexture2DData> PaintTargetData;

private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshPaintSelectionMechanic> SelectionMechanic;

	UPROPERTY(Transient)
	TObjectPtr<UMeshTexturePaintingToolProperties> TextureProperties;

	/** The original texture that we're painting */
	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> PaintingTexture2D;

	/** Hold the transaction while we are painting */
	TUniquePtr<FScopedTransaction> PaintingTransaction;

	double InitialMeshArea = 0;
	bool bArePainting = false;
	bool bResultValid = false;
	bool bStampPending = false;
	bool bInDrag = false;
	bool bRequestPaintBucketFill = false;
	bool bRequiresRuntimeVirtualTextureUpdates = false;

	bool bCachedClickRay = false;
	FRay PendingStampRay;
	FRay PendingClickRay;
	FVector2D PendingClickScreenPosition;
	
	TArray<FPaintRayResults> LastPaintRayResults;
	FHitResult LastBestHitResult;

	FOnPaintingFinishedDelegate OnPaintingFinishedDelegate;
};

/**
 * Class for texture color painting tool.
 * This paints to special textures stored on the mesh components.
 * Behavior should be similar to vertex painting (per instance painting stored on components).
 * But painting texture colors instead of vertex colors is a better fit for very dense mesh types such as used by nanite.
 */
UCLASS(MinimalAPI)
class UMeshTextureColorPaintingTool : public UMeshTexturePaintingTool
{
	GENERATED_BODY()

public:
	UE_API UMeshTextureColorPaintingTool();

protected:
	// Begin UInteractiveTool Interface.
	UE_API virtual void Setup() override;
	// End UInteractiveTool Interface.

	// Begin UMeshTexturePaintingTool Interface.
	virtual bool AllowsMultiselect() const override { return true; }
	UE_API virtual bool IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const override;
	UE_API virtual UTexture2D* GetSelectedPaintTexture(UMeshComponent const* InMeshComponent) const override;
	UE_API virtual int32 GetSelectedUVChannel(UMeshComponent const* InMeshComponent) const override;
	UE_API virtual void GetModifiedTexturesToSave(TArray<UObject*>& OutTexturesToSave) const override;
	UE_API virtual void CacheTexturePaintData() override;
	UE_API virtual bool CanPaintTextureToComponent(UTexture* InTexture, UMeshComponent const* InMeshComponent) const override;
	// End UMeshTexturePaintingTool Interface.

private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshTextureColorPaintingToolProperties> ColorProperties;

	UPROPERTY(Transient)
	TObjectPtr<UTexture> MeshPaintDummyTexture;
};

/**
 * Class for texture asset painting tool.
 * This paints to texture assets directly from the mesh.
 * The texture asset to paint is selected from the ones referenced in the mesh component's materials.
 */
UCLASS(MinimalAPI)
class UMeshTextureAssetPaintingTool : public UMeshTexturePaintingTool
{
	GENERATED_BODY()

public:
	UE_API UMeshTextureAssetPaintingTool();
	
	/** Change selected texture to previous or next available. */
	UE_API void CycleTextures(int32 Direction);

	/** Get the selected paint texture, and return the modified overriden texture if currently painting. */
	UE_API UTexture* GetSelectedPaintTextureWithOverride() const;

	/** Returns true if asset shouldn't be shown in UI because it is not in our paintable texture array. */
	UE_API bool ShouldFilterTextureAsset(const FAssetData& AssetData) const;

	UE_API virtual int32 GetSelectedUVChannel(UMeshComponent const* InMeshComponent) const override;

protected:
	// Begin UInteractiveTool Interface.
	UE_API virtual void Setup() override;
	// End UInteractiveTool Interface.

	// Begin UMeshTexturePaintingTool Interface.
	virtual bool AllowsMultiselect() const override { return true; }
	UE_API virtual bool IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	UE_API virtual UTexture2D* GetSelectedPaintTexture(UMeshComponent const* InMeshComponent) const override;
	UE_API virtual void GetModifiedTexturesToSave(TArray<UObject*>& OutTexturesToSave) const override;
	UE_API virtual void CacheTexturePaintData() override;
	UE_API virtual bool CanPaintTextureToComponent(UTexture* InTexture, UMeshComponent const* InMeshComponent) const override;
	UE_API virtual void ApplyWorldChannelUVs(TArrayView<FTexturePaintTriangleInfo> TriangleInfo) const override;
	// End UMeshTexturePaintingTool Interface.

private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshTextureAssetPaintingToolProperties> AssetProperties;
};

#undef UE_API
