// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/BaseBrushTool.h"
#include "BaseMeshPaintingToolProperties.h"
#include "MeshPaintingToolsetTypes.h"
#include "MeshPaintInteractions.h"
#include "MeshVertexPaintingTool.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

enum class EMeshPaintModeAction : uint8;
enum class EToolShutdownType : uint8;
struct FPerVertexPaintActionArgs;
struct FToolBuilderState;
class IMeshPaintComponentAdapter;

struct FPaintRayResults
{
	FMeshPaintParameters Params;
	FHitResult BestTraceResult;
};

UENUM()
enum class EMeshPaintWeightTypes : uint8
{
	/** Lerp Between Two Textures using Alpha Value */
	AlphaLerp = 2 UMETA(DisplayName = "Alpha (Two Textures)"),

	/** Weighting Three Textures according to Channels*/
	RGB = 3 UMETA(DisplayName = "RGB (Three Textures)"),

	/**  Weighting Four Textures according to Channels*/
	ARGB = 4 UMETA(DisplayName = "ARGB (Four Textures)"),

	/**  Weighting Five Textures according to Channels */
	OneMinusARGB = 5 UMETA(DisplayName = "ARGB - 1 (Five Textures)")
};

UENUM()
enum class EMeshPaintTextureIndex : uint8
{
	TextureOne = 0,
	TextureTwo,
	TextureThree,
	TextureFour,
	TextureFive
};


/**
 *
 */
UCLASS(MinimalAPI)
class UMeshVertexColorPaintingToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};

UCLASS(MinimalAPI)
class UMeshVertexWeightPaintingToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


UCLASS(MinimalAPI)
class UMeshVertexPaintingToolProperties : public UMeshPaintingToolProperties
{
	GENERATED_BODY()

public:
	UE_API UMeshVertexPaintingToolProperties();

	/** When unchecked the painting on the base LOD will be propagate automatically to all other LODs when exiting the mode or changing the selection */
	UPROPERTY(EditAnywhere, Category = VertexPainting, meta = (InlineEditConditionToggle, TransientToolProperty))
	bool bPaintOnSpecificLOD = false;

	/** Index of LOD to paint. If not set then paint is applied to all LODs. */
	UPROPERTY(EditAnywhere, Category = VertexPainting, meta = (UIMin = "0", ClampMin = "0", EditCondition = "bPaintOnSpecificLOD", TransientToolProperty))
	int32 LODIndex = 0;

	/** Size of vertex points drawn when mesh painting is active. */
	UPROPERTY(EditAnywhere, Category = VertexPainting, meta = (UIMin = "0", ClampMin = "0"))
	float VertexPreviewSize;
};

UCLASS(MinimalAPI)
class UMeshVertexColorPaintingToolProperties : public UMeshVertexPaintingToolProperties
{
	GENERATED_BODY()

public:
	/** Whether or not to apply Vertex Color Painting to the Red Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, DisplayName = "Red")
	bool bWriteRed = true;

	/** Whether or not to apply Vertex Color Painting to the Green Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, DisplayName = "Green")
	bool bWriteGreen = true;

	/** Whether or not to apply Vertex Color Painting to the Blue Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, DisplayName = "Blue")
	bool bWriteBlue = true;

	/** Whether or not to apply Vertex Color Painting to the Alpha Channel */
	UPROPERTY(EditAnywhere, Category = ColorPainting, DisplayName = "Alpha")
	bool bWriteAlpha = false;
};

UCLASS(MinimalAPI)
class UMeshVertexWeightPaintingToolProperties : public UMeshVertexPaintingToolProperties
{
	GENERATED_BODY()

public:
	UE_API UMeshVertexWeightPaintingToolProperties();

	/** Texture Blend Weight Painting Mode */
	UPROPERTY(EditAnywhere, Category = WeightPainting, meta = (EnumCondition = 1))
	EMeshPaintWeightTypes TextureWeightType;

	/** Texture Blend Weight index which should be applied during Painting */
	UPROPERTY(EditAnywhere, Category = WeightPainting, meta = (EnumCondition = 1))
	EMeshPaintTextureIndex PaintTextureWeightIndex;

	/** Texture Blend Weight index which should be erased during Painting */
	UPROPERTY(EditAnywhere, Category = WeightPainting, meta = (EnumCondition = 1))
	EMeshPaintTextureIndex EraseTextureWeightIndex;
};


UCLASS(MinimalAPI, Abstract)
class UMeshVertexPaintingTool : public UBaseBrushTool, public IMeshPaintSelectionInterface
{
	GENERATED_BODY()

public:
	UE_API UMeshVertexPaintingTool();

	UE_API void PaintLODChanged();
	UE_API void LODPaintStateChanged(const bool bLODPaintingEnabled);

	UE_API int32 GetMaxLODIndexToPaint() const;
	int32 GetCachedLODIndex() const { return CachedLODIndex; }

	UE_API void CycleMeshLODs(int32 Direction);

	FSimpleDelegate& OnPaintingFinished() { return OnPaintingFinishedDelegate; }

protected:
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual	void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual	bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	virtual bool AllowsMultiselect() const override { return true; }
	UE_API virtual bool IsMeshAdapterSupported(TSharedPtr<IMeshPaintComponentAdapter> MeshAdapter) const override;
	UE_API virtual double EstimateMaximumTargetDimension() override;

	virtual void SetAdditionalPaintParameters(FMeshPaintParameters& InPaintParameters) {};

private:
	UE_API void CacheSelectionData();
	UE_API void ApplyForcedLODIndex(int32 ForcedLODIndex);
	UE_API void UpdateResult();
	UE_API bool Paint(const FVector& InRayOrigin, const FVector& InRayDirection);
	UE_API bool Paint(const TArrayView<TPair<FVector, FVector>>& Rays);
	UE_API bool PaintInternal(const TArrayView<TPair<FVector, FVector>>& Rays, EMeshPaintModeAction PaintAction, float PaintStrength);
	UE_API void ApplyVertexData(FPerVertexPaintActionArgs& InArgs, int32 VertexIndex, FMeshPaintParameters Parameters);
	UE_API void FinishPainting();
	UE_API double CalculateTargetEdgeLength(int TargetTriCount);

private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshPaintSelectionMechanic> SelectionMechanic;

	UPROPERTY(Transient)
	TObjectPtr<UMeshVertexPaintingToolProperties> VertexProperties;

	/** Current LOD index used for painting / forcing */
	int32 CachedLODIndex;
	/** Whether or not a specific LOD level should be forced */
	bool bCachedForceLOD;

	double InitialMeshArea = 0;
	bool bArePainting = false;
	bool bResultValid = false;
	bool bStampPending = false;
	bool bInDrag = false;
	
	bool bCachedClickRay = false;
	FRay PendingStampRay;
	FRay PendingClickRay;
	FVector2D PendingClickScreenPosition;
	FHitResult LastBestHitResult;
	
	FSimpleDelegate OnPaintingFinishedDelegate;
};

UCLASS(MinimalAPI)
class UMeshVertexColorPaintingTool : public UMeshVertexPaintingTool
{
	GENERATED_BODY()

public:
	UE_API UMeshVertexColorPaintingTool();

protected:
	UE_API virtual void Setup() override;

	UE_API virtual void SetAdditionalPaintParameters(FMeshPaintParameters& InPaintParameters) override;
	
private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshVertexColorPaintingToolProperties> ColorProperties;
};

UCLASS(MinimalAPI)
class UMeshVertexWeightPaintingTool : public UMeshVertexPaintingTool
{
	GENERATED_BODY()

public:
	UE_API UMeshVertexWeightPaintingTool();

protected:
	UE_API virtual void Setup() override;
	
	UE_API virtual void SetAdditionalPaintParameters(FMeshPaintParameters& InPaintParameters);

private:
	UPROPERTY(Transient)
	TObjectPtr<UMeshVertexWeightPaintingToolProperties> WeightProperties;
};

#undef UE_API
