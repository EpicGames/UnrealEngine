// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BatchedElements.h"
#include "MeshPaintRendering.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TexturePaintToolset.generated.h"

#define UE_API MESHPAINTINGTOOLSET_API

class FTexture;

class UMeshComponent;
class UTexture;
class UTexture2D;
class UTextureRenderTarget2D;

struct FPaintTexture2DData;

/** Batched element parameters for texture paint shaders used for paint blending and paint mask generation */
class FMeshPaintBatchedElementParameters : public FBatchedElementParameters
{
public:
	/** Binds vertex and pixel shaders for this element */
	virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override
	{
		MeshPaintRendering::SetMeshPaintShaders(RHICmdList, GraphicsPSOInit, InFeatureLevel, InTransform, InGamma, ShaderParams);
	}

public:

	/** Shader parameters */
	MeshPaintRendering::FMeshPaintShaderParameters ShaderParams;
};

/** Batched element parameters for texture paint shaders used for texture dilation */
class FMeshPaintDilateBatchedElementParameters : public FBatchedElementParameters
{
public:
	/** Binds vertex and pixel shaders for this element */
	virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override
	{
		MeshPaintRendering::SetMeshPaintDilateShaders(RHICmdList, GraphicsPSOInit, InFeatureLevel, InTransform, InGamma, ShaderParams);
	}

public:

	/** Shader parameters */
	MeshPaintRendering::FMeshPaintDilateShaderParameters ShaderParams;
};

/** Helper struct to store mesh section information in*/
struct FTexturePaintMeshSectionInfo
{
	/** First vertex index in section */
	int32 FirstIndex;

	/** Last vertex index in section */
	int32 LastIndex;
};

class IMeshPaintComponentAdapter;
struct FPaintableTexture;

/** Helpers functions for texture painting functionality */
UCLASS(MinimalAPI)
class UTexturePaintToolset : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** Static: Copies a texture to a render target texture */
	static UE_API void CopyTextureToRenderTargetTexture(UTexture* SourceTexture, UTextureRenderTarget2D* RenderTargetTexture, ERHIFeatureLevel::Type FeatureLevel);

	/** Will generate a mask texture, used for texture dilation, and store it in the passed in render target */
	static UE_API bool GenerateSeamMask(UMeshComponent* MeshComponent, int32 UVSet, UTextureRenderTarget2D* SeamRenderTexture, UTexture2D* Texture, UTextureRenderTarget2D* RenderTargetTexture );

	/** Returns the maximum bytes per pixel that are supported for source textures when painting. This limitation is set by CreateTempUncompressedTexture. */
	static UE_API int32 GetMaxSupportedBytesPerPixelForPainting();

	/** Returns the pixel format that CreateTempUncompressedTexture uses to create render target data for painting. */
	static UE_API EPixelFormat GetTempUncompressedTexturePixelFormat();

	/** Static: Creates a temporary texture used to transfer data to a render target in memory */
	static UE_API UTexture2D* CreateScratchUncompressedTexture(UTexture2D* SourceTexture);

	/** Keep old legacy method of initializing render target data for the paint brush texture; @todo MeshPaint: Migrate to the method with texture re-use */
	static UE_API void SetupInitialRenderTargetData(UTexture2D* InTextureSource, UTextureRenderTarget2D* InRenderTarget);

	/** Tries to find Materials using the given Texture and retrieve the corresponding material indices from the MEsh Compon*/
	static UE_API void FindMaterialIndicesUsingTexture(const UTexture* Texture, const UMeshComponent* MeshComponent, TArray<int32>& OutIndices);
	
	/** Retrieve LOD mesh sections from MeshComponent which use one of the given textures */
	static UE_API void RetrieveMeshSectionsForTextures(const UMeshComponent* MeshComponent, int32 LODIndex, TArray<const UTexture*> Textures, TArray<FTexturePaintMeshSectionInfo>& OutSectionInfo);

	/** Retrieve LOD mesh sections from MeshComponent which contain one of the Material Indices */
	static UE_API void RetrieveMeshSectionsForMaterialIndices(const UMeshComponent* MeshComponent, int32 LODIndex, const TArray<int32>& MaterialIndices, TArray<FTexturePaintMeshSectionInfo>& OutSectionInfo);

	/** Retrieves all Paintable Textures from the given MeshComponent */
	static UE_API void RetrieveTexturesForComponent(const UMeshComponent* Component, IMeshPaintComponentAdapter* Adapter, int32& OutDefaultIndex, TArray<FPaintableTexture>& OutTextures);
};

#undef UE_API
